[CoreRedirects]
; Converting output to abstract skeleton runtime
+StructRedirects=(OldName="/Script/UAFAnimNode.RigUnit_RunAnimNode", NewName="/Script/UAFAnimNode.RigUnit_RunAnimNode_v1")
+StructRedirects=(OldName="/Script/UAFAnimNode.RunAnimNodeWorkData", NewName="/Script/UAFAnimNode.RunAnimNodeWorkData_v1")
