// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataEx.h"
#include "UAF/AnimNodeCore/UAFAnimNodeReferenceCollector.h"

#include "RigUnit_RunAnimNode_v1.generated.h"

#define UE_API UAFANIMNODE_API

// TODO: Remove me once old content has been resaved
USTRUCT()
struct FUAFAnimNodeReference
{
	GENERATED_BODY()
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input, ExcludeBaseStruct))
	TInstancedStruct<UE::UAF::FUAFAnimNodeData> NodeData;
};

USTRUCT()
struct FRunAnimNodeWorkData_v1
{
	GENERATED_BODY()

	UE_API ~FRunAnimNodeWorkData_v1();

	UE::UAF::FUAFAnimNodePtr AnimNodeInstance;

	TSharedPtr<UE::UAF::FUAFAnimNodeReferenceCollector> GCReferences;

	uint64 DebugInstanceId = 0;
};

/** Runs an AnimNode */
USTRUCT(meta=(Hidden, DisplayName="Run Anim Node (Deprecated)", Category="Anim Node", NodeColor="0, 1, 1", Keywords="Animation, Node, Graph, AnimNode"))
struct FRigUnit_RunAnimNode_v1 : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	// The anim node to run
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	UE::UAF::FUAFAnimNodeDataEx AnimNode;

	// LOD to run at. If this is -1 then the reference pose's source LOD will be used
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	int32 LOD = -1;

	// Reference pose
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FAnimNextGraphReferencePose ReferencePose;

	// The name of the attribute set to execute with, if 'None' will include everything
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FName AttributeSet;

	// Pose result
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Output))
	FAnimNextGraphLODPose Result;

	// Internal work data
	UPROPERTY(transient)
	FRunAnimNodeWorkData_v1 WorkData;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API
