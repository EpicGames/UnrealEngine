// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataEx.h"
#include "UAF/AnimNodeCore/UAFAnimNodeReferenceCollector.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#include "RigUnit_RunAnimNode_v2.generated.h"

#define UE_API UAFANIMNODE_API

USTRUCT()
struct FRunAnimNodeWorkData_v2
{
	GENERATED_BODY()

	UE_API ~FRunAnimNodeWorkData_v2();

	UE::UAF::FUAFAnimNodePtr AnimNodeInstance;

	TSharedPtr<UE::UAF::FUAFAnimNodeReferenceCollector> GCReferences;

	uint64 DebugInstanceId = 0;
};

/** Runs an AnimNode */
USTRUCT(meta=(DisplayName="Run Anim Node", Category="Anim Node", NodeColor="0, 1, 1", Keywords="Animation, Node, Graph, AnimNode"))
struct FRigUnit_RunAnimNode_v2 : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	// The anim node to run
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	UE::UAF::FUAFAnimNodeDataEx AnimNode;

	// The name of the attribute set to execute with, if 'None' will include everything
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Input))
	FName AttributeSet;

	// Result value
	UPROPERTY(EditAnywhere, Category = "Run", meta = (Output))
	FUAFValueBundle Result;

	// Internal work data
	UPROPERTY(transient)
	FRunAnimNodeWorkData_v2 WorkData;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API
