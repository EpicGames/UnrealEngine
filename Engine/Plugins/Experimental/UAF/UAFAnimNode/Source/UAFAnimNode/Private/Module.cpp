// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Animation/AnimSequence.h"
#include "Asset/UAFAnimGraphAssetData.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UAF/UAFAssetData.h"

#include "UAF/AnimNodeCore/UAFAnimNodeFactory.h"
#include "UAF/AnimNodeCore/UAFGraphFactoryAssetAnimNodeFactory.h"
#include "UAF/AnimNodes/UAFSequencePlayer.h"

namespace UE::UAF::AnimNode
{

class FModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		SequencePath = FUAFAnimNodeFactory::RegisterAsset(UAnimSequence::StaticClass(),
			[](UObject* Object, FUAFAnimGraphUpdateContext& Context)
				{
					UAnimSequence* Sequence = CastChecked<UAnimSequence>(Object);
					return MakeAnimNode<FUAFSequencePlayer>(Context, Sequence);
				});
		
		SequenceStructPath = FUAFGraphFactoryAssetAnimNodeFactory::RegisterStruct(FUAFGraphFactoryAsset_Animation::StaticStruct() ,
			[](TConstStructView<FUAFGraphFactoryAsset> Struct, FUAFAnimGraphUpdateContext& Context)
				{
					const FUAFGraphFactoryAsset_Animation& Animation = Struct.Get<FUAFGraphFactoryAsset_Animation>();
					return MakeAnimNode<FUAFSequencePlayer>(Context, Animation.AnimationSequence, Animation.LoopMode);
				});
	}

	virtual void ShutdownModule() override
	{
		FUAFAnimNodeFactory::UnregisterAsset(SequencePath);
		FUAFGraphFactoryAssetAnimNodeFactory::UnregisterStruct(SequenceStructPath);
	}

private:
	FTopLevelAssetPath SequencePath;
	FTopLevelAssetPath SequenceStructPath;
	
};

}

IMPLEMENT_MODULE(UE::UAF::AnimNode::FModule, UAFAnimNode)
