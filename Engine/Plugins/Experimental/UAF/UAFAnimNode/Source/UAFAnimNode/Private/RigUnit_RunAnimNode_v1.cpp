// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode/RigUnit_RunAnimNode_v1.h"

#include "Engine/SkeletalMesh.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/UAFNotifyDispatcherComponent.h"
#include "UAF/UAFSkeletonUserData.h"
#include "UAFLogging.h"
#include "UAF/AnimNodeCore/UAFAnimNodeTrace.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "ObjectTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_RunAnimNode_v1)

FRunAnimNodeWorkData_v1::~FRunAnimNodeWorkData_v1()
{
	// Clear our instance before we destroy the reference collector
	if (AnimNodeInstance)
	{
		UE::UAF::FUAFAnimGraphUpdateContext Context(nullptr, nullptr, DebugInstanceId, *GCReferences, 0.0f); 
		AnimNodeInstance = nullptr;
	}

	GCReferences = nullptr;
}

FRigUnit_RunAnimNode_v1_Execute()
{
	SCOPED_NAMED_EVENT(UAF_Run_AnimNode, FColor::Orange);

	using namespace UE::UAF;

	const FAnimNextModuleContextData& ModuleContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
	FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();

	if (!ReferencePose.ReferencePose.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not run Anim Node - Ref Pose is invalid]")); 
		return;
	}
	
	if (!AnimNode.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not run Anim Node - No Anim Node Selected]")); 
		return;
	}

#if UAF_TRACE_ENABLED
	if (WorkData.DebugInstanceId == 0)
	{
		// Trace an Instance to be the outer containing track in rewind debugger for all the node instance tracks
		// Using this RigUnit's struct type so we can register a custom track details panel to that type in future
		WorkData.DebugInstanceId = FObjectTrace::AllocateInstanceId();
		TRACE_INSTANCE(ModuleContextData.GetObject(), WorkData.DebugInstanceId, ModuleInstance.GetUniqueId(), FRigUnit_RunAnimNode_v1::StaticStruct(), "UAF Anim Graph");
	}
#endif

	const UE::UAF::FReferencePose& RefPose = ReferencePose.ReferencePose.GetRef<UE::UAF::FReferencePose>();

	int32 DesiredLOD = LOD;
	if (DesiredLOD == -1)
	{
		DesiredLOD = RefPose.GetSourceLODLevel();
	}

	// TODO: Currently forcing additive flag to false here
	if (Result.LODPose.ShouldPrepareForLOD(RefPose, DesiredLOD, false))
	{
		Result.LODPose.PrepareForLOD(RefPose, DesiredLOD, true, false);
	}

	ensure(Result.LODPose.LODLevel == DesiredLOD);

	// Internally we use memstack allocation, so we need a mark here
	FMemStack& MemStack = FMemStack::Get();
	FMemMark MemMark(MemStack);

	// We allocate a dummy buffer to trigger the allocation of a large chunk if this is the first mark
	// This reduces churn internally by avoiding a chunk to be repeatedly allocated and freed as we push/pop marks
	MemStack.Alloc(size_t(FPageAllocator::SmallPageSize) + 1, 16);

	if (WorkData.GCReferences == nullptr)
	{
		WorkData.GCReferences = MakeShared<FUAFAnimNodeReferenceCollector>();
	}

	FUAFAnimGraphUpdateContext Context(ModuleContextData.GetObject(), &ModuleInstance, WorkData.DebugInstanceId, *WorkData.GCReferences, ExecuteContext.GetDeltaTime());

	if (WorkData.AnimNodeInstance == nullptr)
	{
		// allocate AnimNode instance
		WorkData.AnimNodeInstance = AnimNode.Get()->CreateInstance(Context);
	}

	TRACE_UAF_MODULE(ModuleInstance);

	if (WorkData.AnimNodeInstance)
	{
		const TArray<FUAFAnimOp*> AnimOpList = UpdateGraph(Context, *WorkData.AnimNodeInstance);

		UAF_TRACE_ANIMOPS(AnimOpList, ModuleInstance, ModuleInstance.GetObject());

		// Use AnimOps to compute our pose
		{
			FUAFSystemOutputComponent& OutputPoseComponent = ModuleInstance.GetOrAddComponent<FUAFSystemOutputComponent>();

			// Note that the skeletal mesh and skeleton can be null
			USkeletalMesh* SkeletalMesh = OutputPoseComponent.GetSkeletalMesh();
			USkeleton* Skeleton = SkeletalMesh != nullptr ? SkeletalMesh->GetSkeleton() : nullptr;

			if (Skeleton != nullptr)
			{
				TNonNullPtr<UUAFSkeletonUserData> UserData = UUAFSkeletonUserData::FromSkeleton(Skeleton);
				UAbstractSkeletonSetBinding* Binding = UserData->GetSetBinding();

				FUAFEvaluateValuesArgs EvaluateArgs(*Binding, AttributeSet, SkeletalMesh, DesiredLOD, AnimOpList, RefPose);

				EvaluateValues(EvaluateArgs, Result);
			}
			else
			{
				// If we have no skeletal mesh or skeleton, we output nothing
			}
		}

		// Now that everything has updated and we've computed our values, we know for certain every blend weight
		
		// Use AnimOps to evaluate our notifies
		{
			TArray<FAnimNotifyEventReference> Notifies = EvaluateNotifies(AnimOpList);
			if (!Notifies.IsEmpty())
			{
				// Ensure we have a handler component on the module
				(void)ModuleInstance.GetOrAddComponent<FUAFNotifyDispatcherComponent>();

				auto NotifyDispatchEvent = MakeTraitEvent<FNotifyDispatchEvent>();
				NotifyDispatchEvent->Notifies = MoveTemp(Notifies);
				NotifyDispatchEvent->Weight = 1.0f;

				ModuleInstance.QueueOutputTraitEvent(NotifyDispatchEvent);
			}
		}

		// Evaluate sync contributors now while the AnimOps are ready, they will be used in the next update
		EvaluateSyncContributors(AnimOpList);
	}
}
