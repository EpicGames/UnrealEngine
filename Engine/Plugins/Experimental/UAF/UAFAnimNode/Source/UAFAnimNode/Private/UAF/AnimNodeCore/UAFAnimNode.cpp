// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "ObjectTrace.h"

namespace UE::UAF
{
	FUAFAnimNode::~FUAFAnimNode()
	{
#if DO_CHECK
		checkf(bIsInitialized, TEXT("Attempting to destroy an anim node that was not initialized. Did you forget to call InitializeAs?"));
#endif

		if (bHasAddReferencedObjects)
		{
			FUAFAnimGraphUpdateContext* UpdateContext = FUAFAnimGraphUpdateContext::GetCurrentFromTLS();
			checkf(UpdateContext != nullptr, TEXT("Attempting to destroy an anim node without a live update context is not allowed."));

			UpdateContext->GetGCReferences().UnregisterWithGC(this);
		}

#if UAF_TRACE_ENABLED
		if (FUAFAnimGraphUpdateContext* UpdateContext = FUAFAnimGraphUpdateContext::GetCurrentFromTLS())
		{
			TRACE_INSTANCE_LIFETIME_END(UpdateContext->GetHostObject(), DebugInstanceId);
		}
#endif
	}

	void FUAFAnimNode::Reset()
	{
#if DO_CHECK
		checkf(Parent == nullptr, TEXT("Attempting to reset a node that has a parent."));
		checkf(!HasPreUpdated(), TEXT("Attempted to reset a node after PreUpdate has been called. It has already updated and queued its work."));
#endif

		// Detach our children and reset everything
		for (const FUAFAnimNodePtr& Child : Children)
		{
			if (Child)
			{
				Child->ClearParent();
			}
		}
		Children.Reset();

		PreAnimOp = nullptr;
		PostAnimOp = nullptr;
		TotalWeight = 0.0f;
		bIsBlendingOut = false;
		bIsNewlyRelevant = true;
	}

	int32 FUAFAnimNode::AddChild(FUAFAnimNodePtr&& Child)
	{
		checkf(!Child || !Children.Contains(Child), TEXT("Attempting to add a duplicate child entry."));

		if (Child)
		{
			checkf(Child->Parent == nullptr, TEXT("Attempting to add a child that already has a parent."));
			Child->SetParent(this);
		}

		return Children.Add(MoveTemp(Child));
	}

	int32 FUAFAnimNode::AddChild(const FUAFAnimNodePtr& Child)
	{
		checkf(!Child || !Children.Contains(Child), TEXT("Attempting to add a duplicate child entry."));

		if (Child)
		{
			checkf(Child->Parent == nullptr, TEXT("Attempting to add a child that already has a parent."));
			Child->SetParent(this);
		}

		return Children.Add(Child);
	}

	bool FUAFAnimNode::RemoveChild(const FUAFAnimNodePtr& Child)
	{
#if DO_CHECK
		checkf(!HasPreUpdated(), TEXT("Attempted to remove a child after PreUpdate has been called. Its children have already updated and queued their work."));
#endif

		if (!Child)
		{
			// Cannot remove unknown child
			return false;
		}

		if (Child->Parent != this)
		{
			// Attempting to remove an unknown child
			return false;
		}

		const int32 ChildIndex = Children.IndexOfByKey(Child);
		check(ChildIndex != INDEX_NONE);

		Child->ClearParent();

		Children.RemoveAt(ChildIndex, EAllowShrinking::No);
		return true;
	}

	void FUAFAnimNode::RemoveChildAt(int32 ChildIndex)
	{
#if DO_CHECK
		checkf(!HasPreUpdated(), TEXT("Attempted to remove a child after PreUpdate has been called. Its children have already updated and queued their work."));
#endif

		if (FUAFAnimNodePtr& Child = Children[ChildIndex])
		{
			Child->ClearParent();
		}

		Children.RemoveAt(ChildIndex, EAllowShrinking::No);
	}

	void FUAFAnimNode::SetChildAt(int32 ChildIndex, const FUAFAnimNodePtr& Child)
	{
		checkf(!Child || !Children.Contains(Child), TEXT("Attempting to add a duplicate child entry."));

		if (FUAFAnimNodePtr& PrevChild = Children[ChildIndex])
		{
#if DO_CHECK
			checkf(!HasPreUpdated(), TEXT("Attempted to replace a child after PreUpdate has been called. Its children have already updated and queued their work."));
#endif			
			PrevChild->ClearParent();
		}

		if (Child)
		{
			checkf(Child->Parent == nullptr, TEXT("Attempting to add a child that already has a parent."));
			Child->SetParent(this);
		}

		Children[ChildIndex] = Child;
	}

	void FUAFAnimNode::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		// Don't inline to ensure that we have a unique branch target for derived types that don't override this
	}

	void FUAFAnimNode::PostUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		// Don't inline to ensure that we have a unique branch target for derived types that don't override this
	}

	void FUAFAnimNode::QueueForDestruction()
	{
		FUAFAnimGraphUpdateContext* UpdateContext = FUAFAnimGraphUpdateContext::GetCurrentFromTLS();
		checkf(UpdateContext != nullptr, TEXT("Attempting to destroy an anim node without a live update context is not allowed."));

#if DO_CHECK
		checkf(!HasPreUpdated(), TEXT("Attempted to destroy an anim node after PreUpdate has been called. It has already updated and queued its work."));
		bIsPendingDestroy = true;
#endif

		// We queue ourself for destruction to avoid recursive tear down
		UpdateContext->QueueForDestruction(this);
	}

#if UAF_TRACE_ENABLED
	FString FUAFAnimNode::GetDebugName() const
	{
		static const FString Name("Anim Node");
		return Name;
	}

	UStruct* FUAFAnimNode::GetDebugStruct() const
	{
		return FUAFAnimNodeData::StaticStruct();
	} 
#endif
}
