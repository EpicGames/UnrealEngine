// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNodeData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFAnimNodeData)

namespace UE::UAF
{
	FUAFAnimNodePtr FUAFAnimNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
	{
		ensureMsgf(false, TEXT("AnimNode CreateInstance Unimplemented!!"));
		return nullptr;
	}
}
