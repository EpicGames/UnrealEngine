// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNodeDataEx.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFAnimNodeDataEx)

namespace UE::UAF
{
#if WITH_EDITORONLY_DATA
	namespace Private
	{
		static bool GShowBakedAnimNodeData = false;
		FAutoConsoleVariableRef CVar_ShowBakedAnimNodeData(TEXT("UAF.ShowBakedAnimNodeData"), GShowBakedAnimNodeData, TEXT("Whether or not to show the baked anim node data within FUAFAnimNodeDataEx"));
	}
#endif

	void FUAFAnimNodeDataEx::Build(const TInstancedStruct<FUAFAnimNodeData>& Base, const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& Modifiers)
	{
		BakedNode = Base;

		for (const TInstancedStruct<FUAFModifierAnimNodeData>& Modifier : Modifiers)
		{
			if (!Modifier.IsValid())
			{
				// Ignore empty modifiers
				continue;
			}

			// Copy our modifier, it will form our new base
			TInstancedStruct<FUAFModifierAnimNodeData> NewBase = Modifier;

#if WITH_EDITORONLY_DATA
			// Show modifier children in the 'BakedData' member for debugging purposes
			NewBase->bShowChildren = true;
#endif

			// Assign our current sub-tree to our new base
			const FProperty* ChildProperty = Modifier.GetScriptStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUAFModifierAnimNodeData, Child));
			check(ChildProperty != nullptr);

			FInstancedStruct* NestedStruct = ChildProperty->ContainerPtrToValuePtr<FInstancedStruct>(NewBase.GetMutableMemory(), 0);
			*NestedStruct = (const FInstancedStruct&)BakedNode;

			// Update our new base
			BakedNode = MoveTemp(NewBase);
		}

#if WITH_EDITORONLY_DATA
		BaseNode = Base;
		ModifierNodes = Modifiers;
		bShowBakedNode = Private::GShowBakedAnimNodeData;

		for (TInstancedStruct<FUAFModifierAnimNodeData>& Modifier : ModifierNodes)
		{
			if (!Modifier.IsValid())
			{
				// Ignore empty modifiers
				continue;
			}

			// Hide modifier children in the modifier list
			Modifier->bShowChildren = false;
		}
#endif
	}

#if WITH_EDITORONLY_DATA
	void FUAFAnimNodeDataEx::Refresh()
	{
		Build(BaseNode, ModifierNodes);
	}

	void FUAFAnimNodeDataEx::PostSerialize(const FArchive& Ar)
	{
		if (Ar.IsLoading())
		{
			Refresh();
		}
	}

	bool FUAFAnimNodeDataEx::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
	{
		// We provide a custom import text in order to be able to refresh our cached data.
		// We disable custom native overrides and call into the default impl.
		const bool bAllowNativeOverride = false;
		const TCHAR* Result = StaticStruct()->ImportText(Buffer, this, Parent, PortFlags, ErrorText, StaticStruct()->GetName(), bAllowNativeOverride);

		Refresh();

		return Result != nullptr;
	}
#endif
}
