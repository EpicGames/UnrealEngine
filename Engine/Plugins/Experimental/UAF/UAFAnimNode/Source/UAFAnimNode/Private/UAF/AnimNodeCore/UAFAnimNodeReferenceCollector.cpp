// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNodeReferenceCollector.h"

namespace UE::UAF
{
	FUAFAnimNodeReferenceCollector::~FUAFAnimNodeReferenceCollector()
	{
		checkf(AnimNodes.IsEmpty(), TEXT("Destroying reference collector while it still holds references"));
	}

	void FUAFAnimNodeReferenceCollector::RegisterWithGC(FUAFAnimNode* AnimNode, AddReferencedObjectsFn GCFn)
	{
		checkf(AnimNode, TEXT("Cannot register a null anim node"));
		checkf(GCFn, TEXT("Cannot register a null GC function"));

		if (!AnimNodes.IsEmpty() && AnimNodes.Last().AnimNode == AnimNode)
		{
			// We are a derived type registering, this is allowed
			AnimNodes.Last().GCFn = GCFn;
		}
		else
		{
			checkf(!AnimNodes.ContainsByPredicate([AnimNode](const FEntry& Entry) { return Entry.AnimNode == AnimNode;  }), TEXT("Cannot register the same anim node twice"));

			AnimNodes.Add(FEntry{ AnimNode, GCFn });
		}
	}

	void FUAFAnimNodeReferenceCollector::UnregisterWithGC(FUAFAnimNode* AnimNode)
	{
		const int32 NodeIndex = AnimNodes.IndexOfByPredicate([AnimNode](const FEntry& Entry) { return Entry.AnimNode == AnimNode;  });
		checkf(NodeIndex != INDEX_NONE, TEXT("Cannot unregister an unknown anim node"));

		AnimNodes.RemoveAtSwap(NodeIndex, EAllowShrinking::No);
	}

	void FUAFAnimNodeReferenceCollector::AddReferencedObjects(FReferenceCollector& Collector)
	{
		for (const FEntry& Entry : AnimNodes)
		{
			(*Entry.GCFn)(Entry.AnimNode, Collector);
		}
	}

	FString FUAFAnimNodeReferenceCollector::GetReferencerName() const
	{
		return TEXT("UAF Anim Node Reference Collector");
	}
}
