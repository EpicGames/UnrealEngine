// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNodeTrace.h"

#include "ObjectAsTraceIdProxyArchive.h"
#include "ObjectTrace.h"
#include "Serialization/MemoryWriter.h"
#include "StructUtils/PropertyBag.h"
#include "UAFAssetInstance.h"
#include "UAF/AnimOpCore/UAFInstancedAnimOpList.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "Animation/AnimTrace.h"

#if UAF_TRACE_ENABLED

UE_TRACE_EVENT_BEGIN(UAFAnimNode, AnimOpList)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, OuterObjectId)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint8[], ListData)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(UAFAnimNode, AnimNodeUpdate)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, RootGraphId)
	UE_TRACE_EVENT_FIELD(uint64, ParentNodeId)
	UE_TRACE_EVENT_FIELD(uint64, NodeId)
	UE_TRACE_EVENT_FIELD(float, TotalWeight)
UE_TRACE_EVENT_END()


UE_TRACE_EVENT_BEGIN(UAFAnimNode, AnimNodeValue)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, RootGraphId)
	UE_TRACE_EVENT_FIELD(uint64, NodeId)
	UE_TRACE_EVENT_FIELD(uint32, NameId)
	UE_TRACE_EVENT_FIELD(uint8, Type)
	UE_TRACE_EVENT_FIELD(uint64, StructType)
	UE_TRACE_EVENT_FIELD(uint8[], Value)
UE_TRACE_EVENT_END()

namespace UE::UAF
{
	void TraceAnimOps(const TArray<FUAFAnimOp*>& AnimOps, const FUAFAssetInstance& RootGraph, const UObject* OuterObject)
	{
		uint64 InstanceId = RootGraph.GetUniqueId();

		double RecordingTime = 0;
		if (OuterObject)
		{
			RecordingTime = FObjectTrace::GetWorldElapsedTime(OuterObject->GetWorld());
		}

		TArray<uint8> ArchiveData;
		FMemoryWriter WriterArchive(ArchiveData);
		FObjectAsTraceIdProxyArchive Archive(WriterArchive);

		static const FUAFInstancedAnimOpList Defaults;
		FUAFInstancedAnimOpList InstancedAnimOpList(AnimOps);
		FUAFInstancedAnimOpList::StaticStruct()->SerializeItem(Archive, &InstancedAnimOpList, &Defaults);

		UE_TRACE_LOG(UAFAnimNode, AnimOpList, UAFChannel)
			<< AnimOpList.Cycle(FPlatformTime::Cycles64())
			<< AnimOpList.OuterObjectId(FObjectTrace::GetObjectId(OuterObject))
			<< AnimOpList.RecordingTime(RecordingTime)
			<< AnimOpList.InstanceId(InstanceId)
			<< AnimOpList.ListData(ArchiveData.GetData(), ArchiveData.Num());

	}

	void TraceAnimNode(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(UAFChannel))
		{
			if (FUAFAnimGraphUpdateContext* UpdateContext = FUAFAnimGraphUpdateContext::GetCurrentFromTLS())
			{
				if (Node->GetDebugInstanceId() == 0)
				{
					Node->SetDebugInstanceId(FObjectTrace::AllocateInstanceId());
				}
				
				TRACE_INSTANCE(UpdateContext->GetHostObject(), Node->GetDebugInstanceId(), UpdateContext->GetOuterDebugInstanceId(), Node->GetDebugStruct(), Node->GetDebugName());

				double RecordingTime = 0;
				if (Context.GetHostObject())
				{
					RecordingTime = FObjectTrace::GetWorldElapsedTime(Context.GetHostObject()->GetWorld());
				}
				uint64 ParentNodeId = 0;
				if (Node->GetParent())
				{
					ParentNodeId = Node->GetParent()->GetDebugInstanceId();
				}

				UE_TRACE_LOG(UAFAnimNode, AnimNodeUpdate, UAFChannel)
					<< AnimNodeUpdate.Cycle(FPlatformTime::Cycles64())
					<< AnimNodeUpdate.RecordingTime(RecordingTime)
					<< AnimNodeUpdate.NodeId(Node->GetDebugInstanceId())
					<< AnimNodeUpdate.ParentNodeId(ParentNodeId)
					<< AnimNodeUpdate.RootGraphId(Context.GetOuterDebugInstanceId())
					<< AnimNodeUpdate.TotalWeight(Node->GetTotalWeight());
			}
		}
	}

	void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, EPropertyBagPropertyType Type, const UScriptStruct* StructType, FName PropertyName, TConstArrayView<uint8> Value)
	{
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(UAFChannel))
		{
			if (FUAFAnimGraphUpdateContext* UpdateContext = FUAFAnimGraphUpdateContext::GetCurrentFromTLS())
			{
				if (Node->GetDebugInstanceId() == 0)
				{
					Node->SetDebugInstanceId(FObjectTrace::AllocateInstanceId());
				}

				double RecordingTime = 0;
				if (Context.GetHostObject())
				{
					RecordingTime = FObjectTrace::GetWorldElapsedTime(Context.GetHostObject()->GetWorld());
				}

				uint32 NameId = FAnimTrace::OutputName(PropertyName);
				uint64 StructId = 0;
				if (StructType)
				{
					TRACE_TYPE(StructType);
					StructId = FObjectTrace::GetObjectId(StructType);
				}
	
				UE_TRACE_LOG(UAFAnimNode, AnimNodeValue, UAFChannel)
					<< AnimNodeValue.Cycle(FPlatformTime::Cycles64())
					<< AnimNodeValue.RecordingTime(RecordingTime)
					<< AnimNodeValue.NodeId(Node->GetDebugInstanceId())
					<< AnimNodeValue.RootGraphId( Context.GetOuterDebugInstanceId())
					<< AnimNodeValue.Type(static_cast<uint8>(Type))
					<< AnimNodeValue.StructType(StructId)
					<< AnimNodeValue.NameId(NameId)
					<< AnimNodeValue.Value(Value.GetData(), Value.Num());
			}
		}
	}

	void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, bool Value)
	{
		TraceAnimNodeValue(Context, Node, EPropertyBagPropertyType::Bool, nullptr, PropertyName, TConstArrayView<uint8>(reinterpret_cast<uint8*>(&Value), sizeof(bool)));
	}

	void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, double Value)
	{
		TraceAnimNodeValue(Context, Node, EPropertyBagPropertyType::Double, nullptr, PropertyName, TConstArrayView<uint8>(reinterpret_cast<uint8*>(&Value), sizeof(double)));
	}

	void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, const FStructView Value)
	{
		if (const UScriptStruct* StructType = Value.GetScriptStruct())
		{
			TArray<uint8> ArchiveData;
			FMemoryWriter WriterArchive(ArchiveData);
			FObjectAsTraceIdProxyArchive Archive(WriterArchive);
			StructType->SerializeBin(Archive, Value.GetMemory());
		
			TraceAnimNodeValue(Context, Node, EPropertyBagPropertyType::Struct, StructType, PropertyName, ArchiveData);
		}
	}

	void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, const UObject* Value)
	{
		uint64 ObjectId = FObjectTrace::GetObjectId(Value);
		TraceAnimNodeValue(Context, Node, EPropertyBagPropertyType::Object, nullptr, PropertyName, TConstArrayView<uint8>(reinterpret_cast<uint8*>(&ObjectId), sizeof(uint64)));
	}
}
#endif
