// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"

#include "Animation/AnimCurveUtils.h"
#include "Graph/AnimNext_LODPose.h"
#include "Misc/MemStack.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimOpCore/UAFAnimOpNotifyEvaluator.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/ValueBundle.h"

namespace UE::UAF
{
	namespace Private
	{
		enum class EUAFUpdateStep : uint8
		{
			PreUpdate,
			PostUpdate,
		};

		// Represents a node during the traversal
		struct FUAFUpdateNodeEntry
		{
			// The node represented by this entry
			FUAFAnimNode* Node = nullptr;

			// Which update step we desire next
			// TODO: We could pack 1 bit in the LSB of our Node pointer for this and avoid the padding
			EUAFUpdateStep DesiredStep = EUAFUpdateStep::PreUpdate;

			// These pointers are mutually exclusive
			// An entry is either part of the pending stack, the free list, or neither
			union
			{
				// Next entry in the linked list of free entries
				FUAFUpdateNodeEntry* NextFreeEntry = nullptr;

				// Previous entry below us in the nodes pending stack
				FUAFUpdateNodeEntry* PrevStackEntry;
			};

			FUAFUpdateNodeEntry(FUAFAnimNode* InNode, EUAFUpdateStep InDesiredStep, FUAFUpdateNodeEntry* InPrevStackEntry = nullptr);
		};

		FUAFUpdateNodeEntry::FUAFUpdateNodeEntry(FUAFAnimNode* InNode, EUAFUpdateStep InDesiredStep, FUAFUpdateNodeEntry* InPrevStackEntry)
			: Node(InNode)
			, DesiredStep(InDesiredStep)
			, PrevStackEntry(InPrevStackEntry)
		{
		}

		// A thread local pointer to the latest update context
		static thread_local FUAFAnimGraphUpdateContext* GCurrentUpdateContext = nullptr;

#if DO_CHECK
		// Debug counter used to enforce invariants
		static std::atomic<uint32> GCurrentAnimNodeUpdateCounter = 1;
#endif
	}

	FUAFAnimGraphUpdateContext::FUAFAnimGraphUpdateContext(UObject* InHostObject, FUAFAssetInstance* InVariablesOwner, uint64 InOuterDebugInstanceId, FUAFAnimNodeReferenceCollector& InGCReferences, float InDeltaTime)
		: PreviousContext(Private::GCurrentUpdateContext)
		, HostObject(InHostObject)
		, VariablesOwner(InVariablesOwner)
		, DeltaTime(InDeltaTime)
		, UpdateDeltaTime(InDeltaTime)
		, GCReferences(InGCReferences)
#if UAF_TRACE_ENABLED
		, OuterDebugInstanceId(InOuterDebugInstanceId)
#endif
	{
		Private::GCurrentUpdateContext = this;

#if DO_CHECK
		UpdateCounter = Private::GCurrentAnimNodeUpdateCounter.fetch_add(1, std::memory_order_relaxed);

		if (UpdateCounter == 0)
		{
			// We wrapped around, skip 0 since it is the default value when nodes are constructed
			UpdateCounter = Private::GCurrentAnimNodeUpdateCounter.fetch_add(1, std::memory_order_relaxed);
		}
#endif
	}

	FUAFAnimGraphUpdateContext::~FUAFAnimGraphUpdateContext()
	{
		checkf(PreviousPlayRateStack.IsEmpty(), TEXT("A play rate was pushed and not popped."));

		// Destroy any nodes that are pending destruction
		// Note that this may queue up more nodes for destruction to avoid recursion
		while (!NodesPendingDestruction.IsEmpty())
		{
			FUAFAnimNode* Node = NodesPendingDestruction.Pop(EAllowShrinking::No);

#if DO_CHECK
			check(Node->bIsPendingDestroy);
#endif

			delete Node;
		}

		Private::GCurrentUpdateContext = PreviousContext;
	}

	FUAFAnimGraphUpdateContext* FUAFAnimGraphUpdateContext::GetCurrentFromTLS()
	{
		return Private::GCurrentUpdateContext;
	}

	void FUAFAnimGraphUpdateContext::PushPlayRate(float PlayRate)
	{
		// Save our current play rate
		PreviousPlayRateStack.Push(CurrentPlayRate);

		// Combine the current play rate with the new one
		CurrentPlayRate *= PlayRate;

		// Update our delta time based on the new play rate
		DeltaTime = UpdateDeltaTime * CurrentPlayRate;
	}

	void FUAFAnimGraphUpdateContext::PopPlayRate()
	{
		checkf(!PreviousPlayRateStack.IsEmpty(), TEXT("Attempting to pop a play rate when none was pushed."));

		// Restore our previous play rate
		CurrentPlayRate = PreviousPlayRateStack.Pop(EAllowShrinking::No);

		// Update our delta time based on the new play rate
		DeltaTime = UpdateDeltaTime * CurrentPlayRate;
	}

	void FUAFAnimGraphUpdateContext::QueueForDestruction(FUAFAnimNode* AnimNode)
	{
		NodesPendingDestruction.Push(AnimNode);
	}

	TArray<FUAFAnimOp*> UpdateGraph(FUAFAnimGraphUpdateContext& UpdateContext, FUAFAnimNode& RootNode)
	{
		using namespace Private;

		SCOPED_NAMED_EVENT(AnimNode_UpdateGraph, FColor::Orange);

		FMemStack& MemStack = FMemStack::Get();
		FMemMark Mark(MemStack);

		TArray<FUAFAnimOp*, TMemStackAllocator<>> AnimOps;
		AnimOps.Reserve(32);

		// Our root always has full weight
		RootNode.SetTotalWeight(1.0f);

		// Add the graph root to kick start the evaluation process
		FUAFUpdateNodeEntry RootEntry(&RootNode, EUAFUpdateStep::PreUpdate);
		FUAFUpdateNodeEntry* NodesPendingUpdateStackTop = &RootEntry;

		// List of free entries we can recycle
		FUAFUpdateNodeEntry* FreeEntryList = nullptr;

		// Process every node twice: pre-update and post-update
		while (NodesPendingUpdateStackTop != nullptr)
		{
			// Grab the top most entry
			FUAFUpdateNodeEntry* Entry = NodesPendingUpdateStackTop;
			FUAFAnimNode* CurrentNode = Entry->Node;

			if (Entry->DesiredStep == EUAFUpdateStep::PreUpdate)
			{
#if DO_CHECK
				checkf(CurrentNode->PreUpdateCounter == CurrentNode->PostUpdateCounter, TEXT("Pre/Post update counters mismatch."));
#endif

				UAF_TRACE_ANIMNODE(UpdateContext, CurrentNode);
				
				// Call PreUpdate before our children have the chance to update
				CurrentNode->PreUpdate(UpdateContext);

#if DO_CHECK
				CurrentNode->PreUpdateCounter = UpdateContext.GetUpdateCounter();
#endif

				// Append the AnimOps that need to execute before those of our children
				// PreAnimOps are uncommon
				if (FUAFAnimOp* AnimOp = CurrentNode->GetPreAnimOp()) [[unlikely]]
				{
					AnimOps.Add(AnimOp);
				}

				// Leave our entry on top of the stack, we'll need to call PostUpdate once the children
				// we'll push on top finish
				Entry->DesiredStep = EUAFUpdateStep::PostUpdate;

				// Append our children in reverse order so that they are visited in the same order they were added
				const TArray<FUAFAnimNodePtr, TInlineAllocator<2>>& Children = CurrentNode->GetChildren();
				
				const int32 NumChildren = Children.Num();
				for (int32 ChildIndex = NumChildren - 1; ChildIndex >= 0; --ChildIndex)
				{
					const FUAFAnimNodePtr& Child = Children[ChildIndex];
					if (!Child) [[unlikely]]
					{
						// Skip null entries
						continue;
					}

					// Insert our new child on top of the stack
					// Re-use an old entry if we can
					FUAFUpdateNodeEntry* ChildEntry;
					if (FreeEntryList != nullptr) [[likely]]
					{
						// Grab an entry from the free list
						ChildEntry = FreeEntryList;
						FreeEntryList = ChildEntry->NextFreeEntry;

						ChildEntry->Node = Child.GetReference();
						ChildEntry->DesiredStep = EUAFUpdateStep::PreUpdate;
						ChildEntry->PrevStackEntry = NodesPendingUpdateStackTop;
					}
					else
					{
						// Allocate a new entry
						ChildEntry = new(MemStack) FUAFUpdateNodeEntry(Child.GetReference(), EUAFUpdateStep::PreUpdate, NodesPendingUpdateStackTop);
					}

					// Most nodes have a single child
					if (NumChildren == 1) [[likely]]
					{
						// If we had a single child, it inherits our blend weight
						Child->SetTotalWeight(CurrentNode->GetTotalWeight());
					}

					// Always inherit whether we are blending out
					Child->SetIsBlendingOut(CurrentNode->IsBlendingOut());

					NodesPendingUpdateStackTop = ChildEntry;
				}

				// Break and continue to the next top-most entry
				// It is either a child ready for its pre-update or our current entry ready for its post-update (leaf)
			}
			else
			{
				// We've already visited this node once, time to post-update
				CurrentNode->PostUpdate(UpdateContext);

#if DO_CHECK
				checkf(CurrentNode->ReferenceCount != 0, TEXT("The last reference to this node was removed during the traversal."));
				CurrentNode->PostUpdateCounter = UpdateContext.GetUpdateCounter();
				checkf(CurrentNode->PreUpdateCounter == CurrentNode->PostUpdateCounter, TEXT("Pre/Post update counters mismatch."));
#endif

				// Append the AnimOps that need to execute after those of our children
				// PostAnimOps are common
				if (FUAFAnimOp* AnimOp = CurrentNode->GetPostAnimOp()) [[likely]]
				{
					AnimOps.Add(AnimOp);
				}
#if DO_CHECK
				else
				{
					checkf(Entry->Node->GetNumChildren() != 0, TEXT("Leaf nodes without children must have a PostAnimOp to produce an output."));
				}
#endif

				// Now that we have been visited, we are no longer newly relevant
				CurrentNode->bIsNewlyRelevant = false;

				// Now that we are done processing this entry, we can pop it
				NodesPendingUpdateStackTop = Entry->PrevStackEntry;

				// This entry is no longer used, add it to the free list
				Entry->NextFreeEntry = FreeEntryList;
				FreeEntryList = Entry;

				// Break and continue to the next top-most entry
				// It is either a sibling ready for its post-update or our parent entry ready for its post-update
			}
		}

		// Now that the root has updated, it is no longer newly relevant
		RootNode.bIsNewlyRelevant = false;

		return TArray<FUAFAnimOp*>(AnimOps);	// Copy with an exact size
	}

	FUAFEvaluateValuesArgs::FUAFEvaluateValuesArgs(UAbstractSkeletonSetBinding& InSkeletonSetBinding, FName InSkeletonSetName, USkeletalMesh* InSkeletalMesh, int32 InLOD, const TArray<FUAFAnimOp*>& InAnimOps, const FReferencePose& InReferencePose)
		: ReferencePose(InReferencePose)
		, AnimOps(InAnimOps)
		, SkeletonSetBinding(&InSkeletonSetBinding)
		, SkeletalMesh(InSkeletalMesh)
		, SkeletonSetName(InSkeletonSetName)
		, LOD(InLOD)
	{
	}

	void EvaluateValues(const FUAFEvaluateValuesArgs& Args, FAnimNextGraphLODPose& OutputValues)
	{
		SCOPED_NAMED_EVENT(AnimNode_EvaluateValues, FColor::Orange);

		const UE::UAF::FReferencePose& RefPose = Args.GetReferencePose();
		const TArray<FUAFAnimOp*>& AnimOps = Args.GetAnimOps();
		const int32 LOD = Args.GetLOD();
		UAbstractSkeletonSetBinding* Binding = Args.GetSkeletonSetBinding();
		FName SetName = Args.GetSkeletonSetName();
		USkeletalMesh* SkeletalMesh = Args.GetSkeletalMesh();

		FUAFAnimOpValueEvaluator Evaluator(Binding, SkeletalMesh, SetName, LOD);

		TUAFStack<FPoseValueBundleCoWRef>& EvaluationStack = Evaluator.GetEvaluationStack();

		for (FUAFAnimOp* AnimOp : AnimOps)
		{
			if (EvaluationStack.Num() < AnimOp->GetNumInputs()) [[unlikely]]
			{
				UE_LOGF(LogAnimation, Warning, "Too few inputs provided, AnimOp will be skipped.");
				continue;
			}

			AnimOp->EvaluateValues(Evaluator);
		}

		const FValueSpace ExpectedValueSpace(EValueSpaceType::Local);
		const FPoseValueBundle* BundleToConvert = nullptr;

		if (!EvaluationStack.IsEmpty()) [[likely]]
		{
			if (EvaluationStack.Num() > 1) [[unlikely]]
			{
				UE_LOGF(LogAnimation, Warning, "Value evaluation produced too many results, first one will be used.");
			}

			const FPoseValueBundleCoWRef* OutputBundleRef = EvaluationStack.Peek();
			const FPoseValueBundle& OutputBundle = OutputBundleRef->Get();
			
			if (!OutputBundle.IsEmpty()) [[likely]]
			{
				if (OutputBundle.GetValueSpace() == ExpectedValueSpace) [[likely]]
				{
					BundleToConvert = &OutputBundle;
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Value evaluation produced a result not in local space, the bind pose will be used.");
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Value evaluation produced an empty result, the bind pose will be used.");
			}
		}

		FPoseValueBundleStack DefaultValueBundle;
		if (BundleToConvert == nullptr)
		{
			DefaultValueBundle.InitWithValueSpace(ExpectedValueSpace);
			BundleToConvert = &DefaultValueBundle;
		}

		// Attributes should no longer be attached to bones in UAF
		// Attributes should have unique names instead
		// Attach to root for now for backwards compatibility
		const FCompactPoseBoneIndex RootCompactPoseBoneIndex(0);

		// Bound value maps
		for (auto It = BundleToConvert->GetBoundValueMaps().CreateConstIterator(); It; ++It)
		{
			const FBoundValueMap* Map = It.GetMap();
			const FAttributeTypedSetPtr& TypedSet = Map->GetTypedSet();

			if (const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Cast<FBoneTransformAnimationAttribute>(Map))
			{
				const int32 NumPoseLODBones = OutputValues.LODPose.GetNumBones();

				for (int32 LODBoneIndex = 0; LODBoneIndex < NumPoseLODBones; ++LODBoneIndex)
				{
					const int32 SkeletonBoneIndex = RefPose.GetSkeletonBoneIndexFromLODBoneIndex(LODBoneIndex);
					const FAttributeBindingIndex BoneBindingIndex = FAttributeBindingIndex(FSkeletonPoseBoneIndex(SkeletonBoneIndex));
					check(BoneBindingIndex.IsValid());

					const FAttributeSetIndex BoneSetIndex = TypedSet->GetIndex(BoneBindingIndex);
					if (BoneSetIndex.IsValid())
					{
						OutputValues.LODPose.LocalTransformsView[LODBoneIndex] = (*BoneTransforms)[BoneSetIndex].Value;
					}
					else
					{
						// This bone isn't present with the current named set, use the bind pose value instead
						OutputValues.LODPose.LocalTransformsView[LODBoneIndex] = RefPose.GetRefPoseTransform(LODBoneIndex);
					}
				}
			}
			else if (const TBoundValueMap<FFloatAnimationAttribute>* Floats = Cast<FFloatAnimationAttribute>(Map))
			{
				const auto GetNameFromIndex = [&TypedSet](int32 CurveIndex)
					{
						return TypedSet->GetName(FAttributeSetIndex(CurveIndex));
					};

				const auto GetValueFromIndex = [Floats](int32 CurveIndex)
					{
						return (*Floats)[FAttributeSetIndex(CurveIndex)].Value;
					};

				// Convert float set attributes into curves
				UE::Anim::FCurveUtils::BuildSorted(OutputValues.Curves, Floats->Num(), GetNameFromIndex, GetValueFromIndex);
			}
			else
			{
				// Everything else
				const UScriptStruct* ValueType = Map->GetValueType();
				const int32 NumAttributes = Map->Num();

				for (FAttributeSetIndex SetIndex(0); SetIndex < NumAttributes; ++SetIndex)
				{
					const UE::Anim::FAttributeId AttributeId(TypedSet->GetName(SetIndex), RootCompactPoseBoneIndex);

					uint8* AttributePtr = OutputValues.Attributes.FindOrAdd(ValueType, AttributeId);

					Map->GetValueWithGetter(SetIndex, [AttributePtr](UScriptStruct* ValueType_, const uint8* ValuePtr)
						{
							ValueType_->CopyScriptStruct(AttributePtr, ValuePtr);
						});
				}
			}
		}

		// Unbound value maps
		for (auto It = BundleToConvert->GetUnboundValueMaps().CreateConstIterator(); It; ++It)
		{
			const FUnboundValueMap* Map = It.GetMap();
			const UScriptStruct* ValueType = Map->GetValueType();
			const int32 NumValues = Map->Num();

			for (int32 ValueIndex = 0; ValueIndex < NumValues; ++ValueIndex)
			{
				const UE::Anim::FAttributeId AttributeId(Map->GetName(ValueIndex), RootCompactPoseBoneIndex);

				uint8* AttributePtr = OutputValues.Attributes.FindOrAdd(ValueType, AttributeId);

				Map->GetValueWithGetter(ValueIndex, [AttributePtr](UScriptStruct* ValueType_, const uint8* ValuePtr)
					{
						ValueType_->CopyScriptStruct(AttributePtr, ValuePtr);
					});
			}
		}
	}

	TArray<FAnimNotifyEventReference> EvaluateNotifies(const TArray<FUAFAnimOp*>& AnimOps)
	{
		SCOPED_NAMED_EVENT(AnimNode_EvaluateNotifies, FColor::Orange);

		FUAFAnimOpNotifyEvaluator Evaluator;

		for (FUAFAnimOp* AnimOp : AnimOps)
		{
			if (!AnimOp->HasEvaluateNotifies()) [[likely]]
			{
				// We skip AnimOps that don't implement EvaluateNotifies since most of them don't
				continue;
			}

			AnimOp->EvaluateNotifies(Evaluator);
		}

		return Evaluator.GetNotifies();
	}

	TArray<FUAFSyncContributor> EvaluateSyncContributors(const TArray<FUAFAnimOp*>& AnimOps)
	{
		SCOPED_NAMED_EVENT(AnimNode_EvaluateSyncContributors, FColor::Orange);

		// TODO: Use AnimOps to compute our phase contributors, we'll need them for the next update
		return TArray<FUAFSyncContributor>();
	}
}
