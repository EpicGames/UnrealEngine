// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFModifierAnimNode.h"

#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"

namespace UE::UAF
{
	FUAFModifierAnimNode::FUAFModifierAnimNode(FUAFAnimGraphUpdateContext& Context)
		: FUAFAnimNode(Context)
	{
		// We reserve a child pointer to simplify the API into get/set
		AddChild(nullptr);
	}

	void FUAFModifierAnimNode::InitializeModifier(FUAFAnimGraphUpdateContext& Context, const FUAFModifierAnimNodeData& ModifierData)
	{
		InitializeModifier(Context, ModifierData.Child);
	}

	void FUAFModifierAnimNode::InitializeModifier(FUAFAnimGraphUpdateContext& Context, const TInstancedStruct<FUAFAnimNodeData>& Child)
	{
		if (const FUAFAnimNodeData* ChildData = Child.GetPtr())
		{
			if (FUAFAnimNodePtr ChildInstance = ChildData->CreateInstance(Context))
			{
				SetChild(MoveTemp(ChildInstance));
			}
		}

		if (!HasChild())
		{
			SetPostAnimOp(FUAFNullAnimOp::Get());
		}
	}

	void FUAFModifierAnimNode::InitializeModifier(FUAFAnimGraphUpdateContext& Context, const FUAFAnimNodePtr& Child)
	{
		if (Child)
		{
			SetChild(Child);
		}
		else
		{
			SetPostAnimOp(FUAFNullAnimOp::Get());
		}
	}

	void* FUAFModifierAnimNode::GetInterface(FUAFAnimNodeInterfaceId Id)
	{
		if (const FUAFAnimNodePtr& Child = GetFirstChild())
		{
			return Child->GetInterface(Id);
		}

		return nullptr;
	}
}
