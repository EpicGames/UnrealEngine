// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFTransitionNodeData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFTransitionNodeData)

namespace UE::UAF
{
	FUAFAnimNodePtr FUAFTransitionNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr Source, FUAFAnimNodePtr Target) const
	{
		ensureMsgf(false, TEXT("FUAFTransitionNodeData CreateInstance Unimplemented!!"));
		return nullptr;
	}
}
