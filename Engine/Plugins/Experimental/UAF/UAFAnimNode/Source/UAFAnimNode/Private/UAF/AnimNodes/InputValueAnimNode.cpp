// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/InputValueAnimNode.h"

#include "Graph/AnimNext_LODPose.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "RemapPoseDataPool.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/PoseValueBundle.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"
#include "UAFAssetInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputValueAnimNode)

namespace UE::UAF
{

void FUAFInputValueAnimOp::ConvertLODPoseToValueBundle(
	const FAnimNextGraphLODPose& InputPose,
	const TReferencePose<FDefaultAllocator, FDefaultSetAllocator>* TargetRefPose,
	FPoseValueBundleStack& OutValueBundle)
{
	const FLODPoseHeap& SourcePose = InputPose.LODPose;
	const FReferencePose* SourceRefPose = SourcePose.RefPose;

	// Determine if we need to remap between different skeletons
	const bool bNeedsRemap = SourceRefPose != nullptr && TargetRefPose != nullptr
		&& FRemapPoseDataPool::NeedsRemapping(*SourceRefPose, *TargetRefPose);

	// The pose we'll actually convert which is either the source directly or a remapped copy.
	const FLODPose* PoseToConvert = &SourcePose;
	FLODPoseHeap RemappedPose;

	if (bNeedsRemap)
	{
		const FRemapPoseData& RemapData = FRemapPoseDataPool::Get().GetRemapData(*SourceRefPose, *TargetRefPose);
		RemapData.RemapPose(SourcePose, RemappedPose);
		PoseToConvert = &RemappedPose;
	}

	OutValueBundle.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));
	FPoseValueBundle& Pose = FPoseValueBundle::From(OutValueBundle);

	// Convert bone transforms
	if (TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Pose.FindBoneTransforms())
	{
		const FAttributeTypedSetPtr& TypedSet = BoneTransforms->GetTypedSet();
		const FReferencePose& RefPose = PoseToConvert->GetRefPose();
		const int32 NumLODBones = PoseToConvert->GetNumBones();

		for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
		{
			const int32 SkeletonBoneIndex = RefPose.GetSkeletonBoneIndexFromLODBoneIndex(LODBoneIndex);
			const FAttributeBindingIndex BoneBindingIndex = FAttributeBindingIndex(FSkeletonPoseBoneIndex(SkeletonBoneIndex));
			if (BoneBindingIndex.IsValid())
			{
				const FAttributeSetIndex BoneSetIndex = TypedSet->GetIndex(BoneBindingIndex);
				if (BoneSetIndex.IsValid())
				{
					(*BoneTransforms)[BoneSetIndex].Value = PoseToConvert->LocalTransformsView[LODBoneIndex];
				}
			}
		}
	}

	// Convert float curves (name-matched, no skeleton dependency)
	if (TBoundValueMap<FFloatAnimationAttribute>* FloatCurves = Pose.FindFloatCurves())
	{
		const FAttributeTypedSetPtr& TypedSet = FloatCurves->GetTypedSet();
		const FBlendedHeapCurve& Curves = InputPose.Curves;

		Curves.ForEachElement([&FloatCurves, &TypedSet](const auto& CurveElement)
		{
			const FAttributeSetIndex CurveSetIndex = TypedSet->FindIndex(CurveElement.Name);
			if (CurveSetIndex.IsValid())
			{
				(*FloatCurves)[CurveSetIndex].Value = CurveElement.Value;
			}
		});
	}
}

// ---------------------------------------------------------------------------
// FUAFInputValueAnimOp
// ---------------------------------------------------------------------------

FUAFInputValueAnimOp::FUAFInputValueAnimOp()
	: FUAFAnimOp(0)
{
	InitializeAs<FUAFInputValueAnimOp>();
}

void FUAFInputValueAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
{
	const FAttributeNamedSetPtr& NamedSet = Evaluator.GetActiveEvaluationContext().GetNamedSet();

	// Direct FValueBundle path, push an immutable reference, CoW handles copying if needed
	// TODO: Verify this path once the system output is an actual value bundle.
	if (CachedBundle && !CachedBundle->IsEmpty())
	{
		Evaluator.GetEvaluationStack().Push(UE::UAF::FPoseValueBundleCoWRef::MakeImmutable(static_cast<const FPoseValueBundle*>(CachedBundle)));
		return;
	}

	// LODPose path, remap if needed, then convert into a FValueBundle
	if (CachedLODPose && CachedLODPose->LODPose.IsValid() && NamedSet)
	{
		FPoseValueBundleStack OutValueBundle(NamedSet);
		ConvertLODPoseToValueBundle(*CachedLODPose, TargetRefPose, OutValueBundle);
		Evaluator.GetEvaluationStack().Push(UE::UAF::FPoseValueBundleCoWRef::MakeFrom(MoveTemp(OutValueBundle)));
		return;
	}

	// Fallback, push an empty bundle (reference pose)
	FPoseValueBundleStack EmptyBundle(NamedSet);
	Evaluator.GetEvaluationStack().Push(UE::UAF::FPoseValueBundleCoWRef::MakeFrom(MoveTemp(EmptyBundle)));
}

// ---------------------------------------------------------------------------
// FUAFInputValueAnimNodeData
// ---------------------------------------------------------------------------

FUAFAnimNodePtr FUAFInputValueAnimNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	return MakeAnimNode<FUAFInputValueAnimNode>(Context, *this);
}

// ---------------------------------------------------------------------------
// FUAFInputValueAnimNode
// ---------------------------------------------------------------------------

FUAFInputValueAnimNode::FUAFInputValueAnimNode(FUAFAnimGraphUpdateContext& Context, const FUAFInputValueAnimNodeData& InData)
	: FUAFAnimNode(Context)
{
	InitializeAs<FUAFInputValueAnimNode>(Context);

	VariableReference = InData.Input;

	InputValueOp.SetDebugOwner(this);
	SetPostAnimOp(&InputValueOp);
}

void FUAFInputValueAnimNode::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
{
	// Clear cached pointers each frame, only valid for one evaluation cycle
	InputValueOp.CachedBundle = nullptr;
	InputValueOp.CachedLODPose = nullptr;
	InputValueOp.TargetRefPose = nullptr;

	FUAFAssetInstance* VariablesOwner = GraphContext.GetVariablesOwner();
	if (!VariablesOwner || VariableReference.IsNone())
	{
		return;
	}

	// Cache the target reference pose from the module's output component.
	// Used during EvaluateValues to remap bones if the source and target skeletons differ.
	if (FAnimNextModuleInstance* ModuleInstance = VariablesOwner->GetRootInstance())
	{
		if (const FUAFSystemOutputComponent* OutputComponent = ModuleInstance->TryGetComponent<FUAFSystemOutputComponent>())
		{
			const FAnimNextGraphReferencePose& GraphRefPose = OutputComponent->GetRefPose();
			if (GraphRefPose.ReferencePose.IsValid())
			{
				InputValueOp.TargetRefPose = &GraphRefPose.ReferencePose.GetRef<TReferencePose<FDefaultAllocator, FDefaultSetAllocator>>();
			}
		}
	}

	// Read the FUAFValueBundle variable and cache whichever representation is available.
	// This must happen during PreUpdate because variable access is only available
	// during the update phase, not during the evaluation phase.
	VariablesOwner->AccessVariable<FUAFValueBundle>(VariableReference, [this](FUAFValueBundle& InValueBundle)
	{
		if (const IVirtualValueBundle* Impl = InValueBundle.GetImpl())
		{
			// Try the direct FValueBundle path first (new attribute runtime)
			if (const FValueBundleHeap* ValueBundle = Impl->GetValueBundle())
			{
				if (!ValueBundle->IsEmpty())
				{
					InputValueOp.CachedBundle = ValueBundle;
					return;
				}
			}

			// Fall back to the LODPose path (old pose system)
			if (const FAnimNextGraphLODPose* LODPose = Impl->GetLODPose())
			{
				if (LODPose->LODPose.IsValid())
				{
					InputValueOp.CachedLODPose = LODPose;
				}
			}
		}
	});
}

#if UAF_TRACE_ENABLED
FString FUAFInputValueAnimNode::GetDebugName() const
{
	return FString::Printf(TEXT("InputValue: %s"), *VariableReference.GetName().ToString());
}

UStruct* FUAFInputValueAnimNode::GetDebugStruct() const
{
	return FUAFInputValueAnimNodeData::StaticStruct();
}
#endif

} // namespace UE::UAF
