// Copyright Epic Games, Inc. All Rights Reserved.


#include "UAF/AnimNodes/UAFApplyAdditive.h"

#include "Logging/StructuredLog.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"


namespace UE::UAF
{
	FUAFAnimNodePtr FUAFApplyAdditiveData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
	{
		return MakeAnimNode<FUAFApplyAdditiveNode>(Context, *this);
	}

	FUAFApplyAdditiveNode::FUAFApplyAdditiveNode(FUAFAnimGraphUpdateContext& Context, const FUAFApplyAdditiveData& InData)
		: FUAFAnimNode(Context)
		, Data(InData)
	{
		InitializeAs<FUAFApplyAdditiveNode>(Context);
		
		AddChild(nullptr);
		AddChild(nullptr);
		
		// Create base and additive child nodes 
		if (Data.BaseNode.IsValid())
		{
			if (const FUAFAnimNodePtr BaseChild = Data.BaseNode.Get().CreateInstance(Context))
			{
				SetChildAt(BaseChildIndex, BaseChild);

				if (Data.AdditiveNode.IsValid())
				{
					if (const FUAFAnimNodePtr AdditiveChild = Data.AdditiveNode.Get().CreateInstance(Context))
					{
						SetChildAt(AdditiveChildIndex, AdditiveChild);
					
						// If both child nodes have been created successfully we create the additive anim op
						const float ActiveAdditiveWeight = Data.AdditiveWeight.GetValue(Context.GetVariablesOwner());
			
						AdditiveAnimOp.SetAdditiveWeight(ActiveAdditiveWeight);
						AdditiveAnimOp.SetBlendMask(Data.BlendMask);
						SetPostAnimOp(&AdditiveAnimOp);
					}
				}
			}
		}
			
		if (GetChildAt(BaseChildIndex) == nullptr)
		{
			// Without base we provide a null op instead
			SetPostAnimOp(FUAFNullAnimOp::Get());
			UE_LOGFMT(LogAnimation, Error, "FUAFApplyAdditiveNode - No valid base input provided. Host Object: [{0}]", GetNameSafe(Context.GetHostObject()));
		}
	
	}
	
	void FUAFApplyAdditiveNode::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		FUAFAnimNode::PreUpdate(GraphContext);

		if (const FUAFAnimNodePtr Base = GetChildAt(BaseChildIndex))
		{
			Base->SetTotalWeight(GetTotalWeight());
			
			if (const FUAFAnimNodePtr Additive = GetChildAt(AdditiveChildIndex))
			{
				const float ActiveAdditiveWeight = Data.AdditiveWeight.GetValue(GraphContext.GetVariablesOwner());
				Additive->SetTotalWeight(GetTotalWeight() * ActiveAdditiveWeight);
				
				// Update relevant values for our AnimOp
				AdditiveAnimOp.SetAdditiveWeight(ActiveAdditiveWeight);
				AdditiveAnimOp.SetBlendMask(Data.BlendMask);
			}
		}
	}

	void* FUAFApplyAdditiveNode::GetInterface(FUAFAnimNodeInterfaceId Id)
	{
		// For now forward any interface calls to the base first and if invalid then to the additive 
		if (const FUAFAnimNodePtr Base = GetChildAt(BaseChildIndex))
		{
			void* BaseInterface = Base->GetInterface(Id);
			if (BaseInterface != nullptr)
			{
				return BaseInterface;
			}
		}
		
		if (const FUAFAnimNodePtr Additive = GetChildAt(AdditiveChildIndex))
		{
			void* AdditiveInterface = Additive->GetInterface(Id);
			if (AdditiveInterface != nullptr)
			{
				return AdditiveInterface;
			}
		}
		
		return nullptr;
	}

	void FUAFApplyAdditiveNode::AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector)
	{
		FUAFApplyAdditiveNode* ThisAdditiveNode = static_cast<FUAFApplyAdditiveNode*>(This);
		if (ThisAdditiveNode)
		{
			FUAFApplyAdditiveData& ThisData = const_cast<FUAFApplyAdditiveData&>(ThisAdditiveNode->Data);
			Collector.AddReferencedObject(ThisData.BlendMask);
			Collector.AddPropertyReferencesWithStructARO(FUAFApplyAdditiveAnimOp::StaticStruct(), &ThisAdditiveNode->AdditiveAnimOp);
		}
	}
	
#if UAF_TRACE_ENABLED
	FString FUAFApplyAdditiveNode::GetDebugName() const
	{
		static FString ApplyAdditiveNodeName("Apply Additive");
		return ApplyAdditiveNodeName;
	}

	UStruct* FUAFApplyAdditiveNode::GetDebugStruct() const
	{
		return FUAFApplyAdditiveData::StaticStruct();
	}
#endif
	
}
