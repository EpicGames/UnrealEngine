// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFBlendByBoolNode.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAFAssetInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFBlendByBoolNode)

namespace UE::UAF
{
FUAFBlendByBoolNode::FUAFBlendByBoolNode(FUAFAnimGraphUpdateContext& Context, const FUAFBlendByBoolNodeData* InData)
	: FUAFBlendStack(Context)
	, Data(InData)
{
	InitializeAs<FUAFBlendByBoolNode>(Context);

	// Read the initial bool value, preferring a binding over the data's literal value.
	bCurrentValue = Data->BoolValue.GetValue(Context.GetVariablesOwner());

	if (bCurrentValue)
	{
		if (Data->TrueNode.IsValid())
		{
			TransitionTo(Context, Data->TrueNode.Get()->CreateInstance(Context));
		}
	}
	else
	{
		if (Data->FalseNode.IsValid())
		{
			TransitionTo(Context, Data->FalseNode.Get()->CreateInstance(Context));
		}
	}
}

void FUAFBlendByBoolNode::PreUpdate(FUAFAnimGraphUpdateContext& Context)
{
	FUAFBlendStack::PreUpdate(Context);

	bool bNewValue = Data->BoolValue.GetValue(Context.GetVariablesOwner());

	UAF_TRACE_ANIMNODE_VALUE(Context, this, "Bool", bNewValue);

	if (bCurrentValue != bNewValue)
	{
		bCurrentValue = bNewValue;
		if (bCurrentValue)
		{
			if (Data->TrueNode.IsValid())
			{
				TransitionTo(Context, Data->TrueNode.Get()->CreateInstance(Context), Data->Transition.GetPtr());
			}
		}
		else
		{
			if (Data->FalseNode.IsValid())
			{
				TransitionTo(Context, Data->FalseNode.Get()->CreateInstance(Context), Data->Transition.GetPtr());
			}
		}
	}

	ValidateSingleChild();
}

#if UAF_TRACE_ENABLED

FString FUAFBlendByBoolNode::GetDebugName() const
{
	static FString BlendByBoolName("Blend By Bool");
	return BlendByBoolName;
}

UStruct* FUAFBlendByBoolNode::GetDebugStruct() const
{
	return FUAFBlendByBoolNodeData::StaticStruct();
}

#endif


FUAFAnimNodePtr FUAFBlendByBoolNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	if (EvaluationFrequency == EBlendByBoolEvaluationFrequency::OnCreate || !BoolValue.HasBinding())
	{
		bool bValue = BoolValue.GetValue(Context.GetVariablesOwner());

		if (bValue)
		{
			if (TrueNode.IsValid())
			{
				return TrueNode.Get()->CreateInstance(Context);
			}
		}
		else
		{
			if (FalseNode.IsValid())
			{
				return FalseNode.Get()->CreateInstance(Context);
			}
		}
		return FUAFAnimNodePtr();
	}

	return MakeAnimNode<FUAFBlendByBoolNode>(Context, this);
}
}
