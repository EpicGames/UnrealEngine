// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFBlendProfileTransition.h"

#include "UAF/BlendProfile/UAFBlendProfile.h"

namespace UE::UAF
{
	FUAFAnimNodePtr FUAFBlendProfileTransitionData::CreateInstance(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr Source, FUAFAnimNodePtr Target) const
	{
		if (Duration <= 0.0f)
		{
			return Target;
		}

		if (!BlendProfile)
		{
			return MakeAnimNode<FUAFSimpleTransition>(Context, Source, Target, Duration, BlendOption);
		}

		return MakeAnimNode<FUAFBlendProfileTransition>(Context, Source, Target, BlendProfile.Get(), Duration, BlendOption);
	}

	FUAFBlendProfileTransition::FUAFBlendProfileTransition(
		FUAFAnimGraphUpdateContext& Context,
		FUAFAnimNodePtr InSource,
		FUAFAnimNodePtr InTarget,
		TNonNullPtr<UUAFBlendProfile> InBlendProfile,
		float InBaseDuration,
		EAlphaBlendOption InBlendOption)
		: FUAFTimedTransition(Context, InSource, InTarget, InBaseDuration, InBlendOption)
		, BlendAnimOp(InBlendProfile, InBlendOption)
	{
		checkf(InSource || InTarget, TEXT("Must have a source and/or target to generate a transition"));
		checkf(InSource != InTarget, TEXT("Cannot transition to/from the same nodes"));
		checkf(InBaseDuration > 0.0f, TEXT("Profile transition must have a positive duration"));
		InitializeAs<FUAFBlendProfileTransition>(Context);

		BlendAnimOp.SetDebugOwner(this);

		SetPostAnimOp(&BlendAnimOp);
	}

	void FUAFBlendProfileTransition::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		SCOPED_NAMED_EVENT(AnimNode_Update_UAFProfileTransitionNode, FColor::Blue);

		FUAFTimedTransition::PreUpdate(GraphContext);
		
		if (!IsComplete())
		{
			const float SourceWeight = TimeRemaining / Duration;
			const float TargetWeight = 1.0f - SourceWeight;

			BlendAnimOp.SetLinearWeightB(TargetWeight);
		}
	}

#if UAF_TRACE_ENABLED
	FString FUAFBlendProfileTransition::GetDebugName() const
	{
		static FString BlendProfileTransitionName("Blend Profile Transition");
		return BlendProfileTransitionName;
	}

	UStruct* FUAFBlendProfileTransition::GetDebugStruct() const
	{
		return FUAFBlendProfileTransitionData::StaticStruct();
	}
#endif
}
