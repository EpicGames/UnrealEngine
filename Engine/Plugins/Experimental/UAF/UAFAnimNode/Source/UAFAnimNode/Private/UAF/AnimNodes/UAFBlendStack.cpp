// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFBlendStack.h"

#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimNodeCore/IUAFTransitionNode.h"
#include "UAF/AnimNodes/UAFSimpleTransition.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"

namespace UE::UAF
{
FUAFBlendStack::FUAFBlendStack(FUAFAnimGraphUpdateContext& Context)
	: FUAFAnimNode(Context)
{
	InitializeAs<FUAFBlendStack>(Context);

	// Reserve a child entry
	AddChild(nullptr);
}

void FUAFBlendStack::PreUpdate(FUAFAnimGraphUpdateContext& Context)
{
	SCOPED_NAMED_EVENT(AnimNode_BlendStack_PreUpdate, FColor::Red);

	PruneTransitions();

	ValidateSingleChild();
}

void FUAFBlendStack::PruneTransitions()
{
	if (bTransitionCompleted)
	{
		if (const FUAFAnimNodePtr& Child = GetChild())
		{
			if (IUAFTransitionNode* TransitionInstance = Child->GetInterface<IUAFTransitionNode>())
			{
				if (TransitionInstance->IsComplete())
				{
					// Our transition is complete, prune it
					FUAFAnimNodePtr Target = TransitionInstance->Prune();
					SetChildAt(0, Target);
				}
			}
		}

		bTransitionCompleted = false;
	}
}
	
void FUAFBlendStack::TransitionTo(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* TargetChild, const FUAFTransitionNodeData* Transition)
{
	if (ensure(TargetChild))
	{
		if (const FUAFAnimNodePtr SourceChild = ReleaseChild())
		{
			FUAFAnimNodePtr TransitionChild;
			if (Transition)
			{
				TransitionChild = Transition->CreateInstance(Context, SourceChild, TargetChild);
			}
			else
			{
				TransitionChild = MakeAnimNode<FUAFSimpleTransition>(Context, SourceChild, TargetChild, 0.1f, EAlphaBlendOption::Linear);
			}

			SetChild(TransitionChild);
		}
		else
		{
			SetChild(TargetChild);
		}
	}
}

#if UAF_TRACE_ENABLED
	
FString FUAFBlendStack::GetDebugName() const
{
	static FString BlendStackName("Blend Stack");
	return BlendStackName;
}

UStruct* FUAFBlendStack::GetDebugStruct() const
{
	return FUAFAnimNodeData::StaticStruct();
}

#endif

void FUAFBlendStack::ValidateSingleChild()
{
	if (GetChild())
	{
		// We have a valid child, we'll simply pass-through
		SetPostAnimOp(nullptr);
	}
	else if (!GetPostAnimOp())
	{
		// We have no valid child, we'll output an empty AnimOp
		SetPostAnimOp(FUAFNullAnimOp::Get());
	}
}

void* FUAFBlendStack::GetInterface(FUAFAnimNodeInterfaceId Id)
{
	if (Id == IUAFTransitionContainerNode::InterfaceId)
	{
		return static_cast<IUAFTransitionContainerNode*>(this);
	}
	
	if (Id == IUAFTransitionNode::InterfaceId)
	{
		// temp fix until we add interface "Search Depth" parameter:
		// because this node may contain Transitions, and we call GetInterface to check for Transitions to prune...
		// make sure the Transition Interface is never passed back from this node's child
		return nullptr;
	}
	
	if (const FUAFAnimNodePtr& Child = GetChild())
	{
		return Child->GetInterface(Id);
	}

	return nullptr;
}

void FUAFBlendStack::NotifyTransitionComplete(const IUAFTransitionNode& TransitionNode)
{
	bTransitionCompleted = true;
}

}
