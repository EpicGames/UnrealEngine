// Copyright Epic Games, Inc. All Rights Reserved.


#include "UAF/AnimNodes/UAFMakeDynamicAdditive.h"

#include "Logging/StructuredLog.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"


namespace UE::UAF
{

	FUAFAnimNodePtr FUAFMakeDynamicAdditiveData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
	{
		return MakeAnimNode<FUAFMakeDynamicAdditiveNode>(Context, *this);
	}

	FUAFMakeDynamicAdditiveNode::FUAFMakeDynamicAdditiveNode(FUAFAnimGraphUpdateContext& Context, const FUAFMakeDynamicAdditiveData& InData)
		: FUAFAnimNode(Context)
		, Data(&InData)
	{
		InitializeAs<FUAFMakeDynamicAdditiveNode>(Context);

		// Create both child nodes up front,
		// Base and Source are both required.
		FUAFAnimNodePtr BaseChild;
		FUAFAnimNodePtr SourceChild;

		if (Data->BaseNode.IsValid())
		{
			BaseChild = Data->BaseNode.Get().CreateInstance(Context);
		}

		if (Data->SourceNode.IsValid())
		{
			SourceChild = Data->SourceNode.Get().CreateInstance(Context);
		}

		if (BaseChild && SourceChild)
		{
			AddChild(BaseChild);
			AddChild(SourceChild);

			MakeDynamicAdditiveAnimOp.SetMeshSpaceAdditive(Data->bMeshSpaceAdditive);
			SetPostAnimOp(&MakeDynamicAdditiveAnimOp);
		}
		else
		{
			SetPostAnimOp(FUAFNullAnimOp::Get());
			UE_LOGFMT(LogAnimation, Error, "FUAFMakeDynamicAdditiveNode - Both Base and Source inputs are required. Host Object: [{0}]", GetNameSafe(Context.GetHostObject()));
		}
	}

	void FUAFMakeDynamicAdditiveNode::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		FUAFAnimNode::PreUpdate(GraphContext);

		check(Data);
		// If I have any children, it means I have both children.
		if (HasChildren())
		{
			// Both inputs have full weight.
			GetChildAt(BaseChildIndex)->SetTotalWeight(GetTotalWeight());
			GetChildAt(SourceChildIndex)->SetTotalWeight(GetTotalWeight());
		}
	}

	void* FUAFMakeDynamicAdditiveNode::GetInterface(FUAFAnimNodeInterfaceId Id)
	{
		if (!HasChildren())
		{
			return nullptr;
		}

		if (const FUAFAnimNodePtr& Source = GetChildAt(SourceChildIndex))
		{
			return Source->GetInterface(Id);
		}

		return nullptr;
	}

	void FUAFMakeDynamicAdditiveNode::AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector)
	{
		FUAFMakeDynamicAdditiveNode* ThisNode = static_cast<FUAFMakeDynamicAdditiveNode*>(This);
		Collector.AddPropertyReferencesWithStructARO(FUAFMakeDynamicAdditiveAnimOp::StaticStruct(), &ThisNode->MakeDynamicAdditiveAnimOp);
	}

#if UAF_TRACE_ENABLED
	FString FUAFMakeDynamicAdditiveNode::GetDebugName() const
	{
		static FString MakeDynamicAdditiveNodeName("Make Dynamic Additive");
		return MakeDynamicAdditiveNodeName;
	}

	UStruct* FUAFMakeDynamicAdditiveNode::GetDebugStruct() const
	{
		return FUAFMakeDynamicAdditiveData::StaticStruct();
	}
#endif
}
