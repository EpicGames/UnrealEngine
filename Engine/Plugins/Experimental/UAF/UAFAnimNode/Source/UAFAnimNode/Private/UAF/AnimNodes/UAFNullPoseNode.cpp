// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFNullPoseNode.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"

namespace UE::UAF
{
	FUAFNullPoseNode::FUAFNullPoseNode(FUAFAnimGraphUpdateContext& Context)
		: FUAFAnimNode(Context)
	{
		InitializeAs<FUAFNullPoseNode>(Context);
		SetPostAnimOp(FUAFNullAnimOp::Get());
	}

#if UAF_TRACE_ENABLED
	FString FUAFNullPoseNode::GetDebugName() const
	{
		return "Null AnimNode";
	}

	UStruct* FUAFNullPoseNode::GetDebugStruct() const
	{
		return FUAFAnimNodeData::StaticStruct();
	}
#endif
}
