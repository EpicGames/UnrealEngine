// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFOffsetRootBoneNode.h"

namespace UE::UAF
{

FUAFAnimNodePtr FUAFOffsetRootBoneNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	return MakeAnimNode<FUAFOffsetRootBoneNode>(Context, *this);
}

FUAFOffsetRootBoneNode::FUAFOffsetRootBoneNode(FUAFAnimGraphUpdateContext& Context, const FUAFOffsetRootBoneNodeData& InData)
	: FUAFModifierAnimNode(Context)
	, Data(&InData)
{
	InitializeAs<FUAFOffsetRootBoneNode>(Context);
	InitializeModifier(Context, InData);

	// Copy non-latent (constant) properties from data to AnimOp
	OffsetRootBoneAnimOp.bClampToTranslationVelocity = InData.bClampToTranslationVelocity;
	OffsetRootBoneAnimOp.bClampToRotationVelocity = InData.bClampToRotationVelocity;
	OffsetRootBoneAnimOp.TranslationSpeedRatio = InData.TranslationSpeedRatio;
	OffsetRootBoneAnimOp.RotationSpeedRatio = InData.RotationSpeedRatio;
	OffsetRootBoneAnimOp.TeleportDistanceThreshold = InData.TeleportDistanceThreshold;
	OffsetRootBoneAnimOp.bResetOnTeleport = InData.bResetOnTeleport;

	OffsetRootBoneAnimOp.HostObject = Context.GetHostObject();
	SetPostAnimOp(&OffsetRootBoneAnimOp);
}

#if UAF_TRACE_ENABLED
FString FUAFOffsetRootBoneNode::GetDebugName() const
{
	static FString Name(TEXT("OffsetRootBone"));
	return Name;
}

UStruct* FUAFOffsetRootBoneNode::GetDebugStruct() const
{
	return FUAFOffsetRootBoneNodeData::StaticStruct();
}
#endif // UAF_TRACE_ENABLED

void FUAFOffsetRootBoneNode::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
{
	OffsetRootBoneAnimOp.DeltaTime = GraphContext.GetDeltaTime();

	// Resolve bindable values
	FUAFAssetInstance* VarOwner = GraphContext.GetVariablesOwner();
	OffsetRootBoneAnimOp.Alpha = Data->Alpha.GetValue(VarOwner);
	OffsetRootBoneAnimOp.MeshComponentTransformWorld = Data->MeshComponentTransformWorld.GetValue(VarOwner);
	OffsetRootBoneAnimOp.TranslationMode = Data->TranslationMode.GetValue<EUAFOffsetRootBoneNodeMode>(VarOwner);
	OffsetRootBoneAnimOp.RotationMode = Data->RotationMode.GetValue<EUAFOffsetRootBoneNodeMode>(VarOwner);
	OffsetRootBoneAnimOp.TranslationSmoothingTime = Data->TranslationSmoothingTime.GetValue(VarOwner);
	OffsetRootBoneAnimOp.RotationSmoothingTime = Data->RotationSmoothingTime.GetValue(VarOwner);
	OffsetRootBoneAnimOp.MaxTranslationError = Data->MaxTranslationError.GetValue(VarOwner);
	OffsetRootBoneAnimOp.MaxRotationErrorDegrees = Data->MaxRotationErrorDegrees.GetValue(VarOwner);
	OffsetRootBoneAnimOp.bOnGround = Data->bOnGround.GetValue(VarOwner);
	OffsetRootBoneAnimOp.AnimatedGroundNormal = Data->AnimatedGroundNormal.GetValue(VarOwner);
	
	FUAFAnimNode::PreUpdate(GraphContext);
}

} // namespace UE::UAF
