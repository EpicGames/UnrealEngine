// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFScalePlayRate.h"

#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFScalePlayRate)

namespace UE::UAF
{
	FUAFScalePlayRate::FUAFScalePlayRate(FUAFAnimGraphUpdateContext& Context, const FUAFScalePlayRateData& Data)
		: FUAFModifierAnimNode(Context)
		, PlayRateMultiplier(Data.PlayRateMultiplier)
	{
		InitializeAs<FUAFScalePlayRate>(Context);
		InitializeModifier(Context, Data);
	}

	void FUAFScalePlayRate::SetPlayRate(float PlayRate)
	{
		PlayRateMultiplier.ClearBinding();
		PlayRateMultiplier.SetConstantValue(PlayRate);
	}

	void FUAFScalePlayRate::PreUpdate(FUAFAnimGraphUpdateContext& Context)
	{
		float Multiplier = PlayRateMultiplier.GetValue(Context.GetVariablesOwner());
		UAF_TRACE_ANIMNODE_VALUE(Context, this, "PlayRateMultiplier", Multiplier);
		Context.PushPlayRate(Multiplier);
	}

	void FUAFScalePlayRate::PostUpdate(FUAFAnimGraphUpdateContext& Context)
	{
		Context.PopPlayRate();
	}

#if UAF_TRACE_ENABLED
	FString FUAFScalePlayRate::GetDebugName() const
	{
		static FString Name("Scale Play Rate");
		return Name;
	}

	UStruct* FUAFScalePlayRate::GetDebugStruct() const
	{
		return FUAFScalePlayRateData::StaticStruct();
	}
#endif

	FUAFAnimNodePtr FUAFScalePlayRateData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
	{
		return MakeAnimNode<FUAFScalePlayRate>(Context, *this);
	}
}
