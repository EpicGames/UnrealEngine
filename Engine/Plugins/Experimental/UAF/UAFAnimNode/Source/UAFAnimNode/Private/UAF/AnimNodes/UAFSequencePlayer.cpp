// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFSequencePlayer.h"

#include "AnimationRuntime.h"
#include "Module/AnimNextModuleInstance.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "UAF/UAFSkeletonUserData.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFSequencePlayer)

namespace UE::UAF
{
	// Return if the sequence player should loop, based on the setting on the Sequence itself, and the LoopMode enum
	bool ShouldLoopSequence(const UAnimSequence* Sequence, EAnimAssetLoopMode LoopMode)
	{
		switch (LoopMode)
		{
		case EAnimAssetLoopMode::Auto:
			return Sequence ? Sequence->bLoop : false;
		case EAnimAssetLoopMode::ForceLoop:
			return true;
		case EAnimAssetLoopMode::ForceNonLoop:
			return false;
		default:
			return false;
		}
	}
	
	FUAFSequencePlayer::FUAFSequencePlayer(FUAFAnimGraphUpdateContext& Context, const FUAFSequencePlayerData* InData)
		: FUAFAnimNode(Context)
	{
		InitializeAs<FUAFSequencePlayer>(Context);
	
		if (InData->Sequence)
		{
			const bool bIsLooping = ShouldLoopSequence(InData->Sequence, InData->LoopMode);

			constexpr bool bInterpolate = true;
			constexpr bool bExtractTrajectory = true;
			DecompressAnimOp.Initialize(InData->Sequence, InData->StartTime, bIsLooping, bInterpolate, bExtractTrajectory);
			DecompressAnimOp.SetDebugOwner(this);

			SetPostAnimOp(&DecompressAnimOp);
		}
		else
		{
			SetPostAnimOp(FUAFNullAnimOp::Get());
		}
	}

	FUAFSequencePlayer::FUAFSequencePlayer(FUAFAnimGraphUpdateContext& Context, const UAnimSequence* InSequence, EAnimAssetLoopMode InLoopMode)
		: FUAFAnimNode(Context)
	{
		InitializeAs<FUAFSequencePlayer>(Context);
	
		if (InSequence)
		{
			const bool bIsLooping = ShouldLoopSequence(InSequence, InLoopMode);

			constexpr bool bInterpolate = true;
			constexpr bool bExtractTrajectory = true;
			DecompressAnimOp.Initialize(InSequence, 0.0f, bIsLooping, bInterpolate, bExtractTrajectory);
			DecompressAnimOp.SetDebugOwner(this);

			SetPostAnimOp(&DecompressAnimOp);
		}
		else
		{
			SetPostAnimOp(FUAFNullAnimOp::Get());
		}
	}

	void FUAFSequencePlayer::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		SCOPED_NAMED_EVENT(AnimNode_Update_SequenceController, FColor::Green);

		UAF_TRACE_ANIMNODE_VALUE(GraphContext, this, "Asset", DecompressAnimOp.GetAnimSequence());

		const float DeltaTime = GraphContext.GetDeltaTime();
		DecompressAnimOp.AdvanceTime(DeltaTime);
	}

	void* FUAFSequencePlayer::GetInterface(FUAFAnimNodeInterfaceId Id)
	{
		if (Id == IUAFAnimNodeTimeline::InterfaceId)
		{
			return static_cast<IUAFAnimNodeTimeline*>(this);
		}

		if (Id == IUAFRootMotionProvider::InterfaceId)
		{
			return static_cast<IUAFRootMotionProvider*>(this);
		}

		return nullptr;
	}

	void FUAFSequencePlayer::AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector)
	{
		FUAFSequencePlayer* That = static_cast<FUAFSequencePlayer*>(This);
		Collector.AddPropertyReferencesWithStructARO(FUAFDecompressAnimSequenceAnimOp::StaticStruct(), &That->DecompressAnimOp);
	}

#if UAF_TRACE_ENABLED
	FString FUAFSequencePlayer::GetDebugName() const
	{
		if (const UAnimSequence* Sequence = DecompressAnimOp.GetAnimSequence())
		{
			return Sequence->GetName();
		}
		else
		{
			return "Sequence Player";
		}
	}

	UStruct* FUAFSequencePlayer::GetDebugStruct() const
	{
		return FUAFSequencePlayerData::StaticStruct();
	}
#endif

	void FUAFSequencePlayer::SetCurrentTime(float InCurrentTime)
	{
		const float DeltaTime = InCurrentTime - DecompressAnimOp.GetCurrentTime();
		DecompressAnimOp.AdvanceTime(DeltaTime);
	}

	ETypeAdvanceAnim FUAFSequencePlayer::GetTimeAdvanceResult() const
	{
		return DecompressAnimOp.GetTimeAdvanceResult();
	}

	FTransform FUAFSequencePlayer::ExtractRootMotion(float StartTime, float DeltaTime, bool bAllowLooping) const
	{
		if (const UAnimSequence* AnimSequence = DecompressAnimOp.GetAnimSequence())
		{
			// We do not check for lifetimes, assume the sequence is alive during pose list execution.
			check(AnimSequence->IsValidLowLevel());

			const bool bIsLooping = IsLooping();

			// Make sure we don't extract outside our valid range
			StartTime = FMath::Clamp(StartTime, 0.0f, AnimSequence->GetPlayLength());

			return AnimSequence->ExtractRootMotion(FAnimExtractContext(static_cast<double>(StartTime), true, FDeltaTimeRecord(DeltaTime), bAllowLooping && bIsLooping));
		}

		return FTransform::Identity;
	}

	bool FUAFSequencePlayer::IsLooping() const
	{
		return DecompressAnimOp.IsLooping();
	}

	float FUAFSequencePlayer::GetCurrentTime() const
	{
		return DecompressAnimOp.GetCurrentTime();
	}

	float FUAFSequencePlayer::GetLength() const
	{
		return DecompressAnimOp.GetDuration();
	}

	FUAFAnimNodePtr FUAFSequencePlayerData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
	{
		return MakeAnimNode<FUAFSequencePlayer>(Context, this);
	}

	void* FUAFSequencePlayerData::GetInterface(FUAFAnimNodeDataInterfaceId Id)
	{
		if (Id == IUAFAnimNodeDataHasAsset::InterfaceId)
		{
			return static_cast<IUAFAnimNodeDataHasAsset*>(this);
		}

		return nullptr;
	}
}
