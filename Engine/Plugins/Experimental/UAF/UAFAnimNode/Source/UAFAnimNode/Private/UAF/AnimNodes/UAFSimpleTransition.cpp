// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFSimpleTransition.h"

#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFSimpleTransition)

namespace UE::UAF
{
	FUAFAnimNodePtr FUAFSimpleTransitionData::CreateInstance(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr Source, FUAFAnimNodePtr Target) const
	{
		if (Duration <= 0.0f)
		{
			return Target;
		}

		return MakeAnimNode<FUAFSimpleTransition>(Context, Source, Target, Duration, BlendOption);
	}

	FUAFSimpleTransition::FUAFSimpleTransition(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr InSource, FUAFAnimNodePtr InTarget, float InDuration, EAlphaBlendOption InBlendOption)
		: FUAFTimedTransition(Context, InSource, InTarget, InDuration, InBlendOption)
	{
		InitializeAs<FUAFSimpleTransition>(Context);

		if (InSource && InTarget && InDuration > 0.0f)
		{
			BlendAnimOp.SetDebugOwner(this);
			BlendAnimOp.SetBlendOption(InBlendOption);
			
			SetPostAnimOp(&BlendAnimOp);
		}
	}

	void FUAFSimpleTransition::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		SCOPED_NAMED_EVENT(AnimNode_Update_UAFBasocTransitionNode, FColor::Blue);

		FUAFTimedTransition::PreUpdate(GraphContext);
		
		if (!IsComplete())
		{
			const float SourceWeight = TimeRemaining / Duration;
			const float TargetWeight = 1.0f - SourceWeight;
			
			BlendAnimOp.SetInterpolationAlpha(TargetWeight);
		}
	}

#if UAF_TRACE_ENABLED
	FString FUAFSimpleTransition::GetDebugName() const
	{
		static FString SimpleTransitionName("Simple Transition");
		return SimpleTransitionName;
	}

	UStruct* FUAFSimpleTransition::GetDebugStruct() const
	{
		return FUAFSimpleTransitionData::StaticStruct();
	}
#endif

}
