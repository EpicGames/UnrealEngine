// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFSteeringNode.h"

#include "UAF/AnimNodes/IUAFRootMotionProvider.h"
#include "UAF/AnimNodes/IUAFAnimNodeTimeline.h"

namespace UE::UAF
{
FUAFAnimNodePtr FUAFSteeringNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	return MakeAnimNode<FUAFSteeringNode>(Context, *this);
}

FUAFSteeringNode::FUAFSteeringNode(FUAFAnimGraphUpdateContext& Context, const FUAFSteeringNodeData& InData)
	: FUAFModifierAnimNode(Context)
	, Data(&InData)
{
	InitializeAs<FUAFSteeringNode>(Context);
	InitializeModifier(Context, InData);
	
	if (FUAFAnimNodePtr Child = GetChild())
	{
		// Update interfaces
		SteeringAnimOp.RootMotionProvider = Child->GetInterface<IUAFRootMotionProvider>();
		SteeringAnimOp.Timeline = Child->GetInterface<IUAFAnimNodeTimeline>();
	}

	SteeringAnimOp.HostObject = Context.GetHostObject();
	SetPostAnimOp(&SteeringAnimOp);
}

#if UAF_TRACE_ENABLED
FString FUAFSteeringNode::GetDebugName() const
{
	static FString SteeringNodeName("Steering");
	return SteeringNodeName;
}

UStruct* FUAFSteeringNode::GetDebugStruct() const
{
	return FUAFSteeringNodeData::StaticStruct();
}
#endif // UAF_TRACE_ENABLED

void FUAFSteeringNode::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
{
	// Update Op with bindable values
	SteeringAnimOp.Alpha = Data->Alpha.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.AnimatedTargetTime = Data->AnimatedTargetTime.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.DisableAdditiveBelowSpeed = Data->DisableAdditiveBelowSpeed.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.DisableSteeringBelowSpeed = Data->DisableSteeringBelowSpeed.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.TargetOrientation = Data->TargetOrientation.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.RootBoneTransform = Data->RootBoneTransform.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.ProceduralTargetTime = Data->ProceduralTargetTime.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.RootMotionAngleThresholdDegrees = Data->RootMotionAngleThresholdDegrees.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.MinScaleRatio = Data->MinScaleRatio.GetValue(GraphContext.GetVariablesOwner());
	SteeringAnimOp.MaxScaleRatio = Data->MaxScaleRatio.GetValue(GraphContext.GetVariablesOwner());
	
	UAF_TRACE_ANIMNODE_VALUE(GraphContext, this, "Alpha", SteeringAnimOp.Alpha);
	UAF_TRACE_ANIMNODE_VALUE(GraphContext, this, "TargetOrientation", FStructView::Make(SteeringAnimOp.TargetOrientation));
	
	FUAFAnimNode::PreUpdate(GraphContext);
}

void FUAFSteeringNode::PostUpdate(FUAFAnimGraphUpdateContext& GraphContext)
{
	FUAFAnimNode::PostUpdate(GraphContext);

	SteeringAnimOp.DeltaTime = GraphContext.GetDeltaTime();

	if (FUAFAnimNodePtr Child = GetChild())
	{
		// Update interfaces: they can change in PreUpdate for e.g. transitions
		SteeringAnimOp.RootMotionProvider = Child->GetInterface<IUAFRootMotionProvider>();
		SteeringAnimOp.Timeline = Child->GetInterface<IUAFAnimNodeTimeline>();
	}
}
}
