// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodes/UAFTimedTransition.h"

#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimNodeCore/UAFTransitionNodeData.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"

namespace UE::UAF
{
	FUAFTimedTransition::FUAFTimedTransition(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr InSource, FUAFAnimNodePtr InTarget, float InDuration, EAlphaBlendOption InBlendOption)
		: FUAFAnimNode(Context)
		, Duration(InDuration)
		, BlendOption(InBlendOption)
		, TimeRemaining(InDuration)
	{
		checkf(InSource || InTarget, TEXT("Must have a source and/or target to generate a transition"));
		InitializeAs<FUAFTimedTransition>(Context);

		SetPostAnimOp(nullptr);
		
		if (InSource && InTarget && InDuration > 0.0f)
		{
			AddChild(InSource);
			AddChild(InTarget);

			InSource->SetIsBlendingOut(true);
		}
		else if (InTarget)
		{
			// We only have a target or our duration is zero, we'll snap to it
			AddChild(nullptr);
			AddChild(InTarget);

			Duration = 0.0f;
			TimeRemaining = 0.0f;
		}
		else
		{
			// We have no target, we cannot transition meaningfully
			// We do not track the source, we'll pop to the bind pose
			AddChild(nullptr);
			AddChild(nullptr);

			Duration = 0.0f;
			TimeRemaining = 0.0f;
		}
	}

	void FUAFTimedTransition::NotifyTransitionComplete(const IUAFTransitionNode& TransitionNode)
	{
		bNestedTransitionComplete = true;
	}

	FUAFAnimNodePtr FUAFTimedTransition::ReleaseSource()
	{
		checkf(IsComplete(), TEXT("Attempting to release ownership of the transition source before it has completed."));

		FUAFAnimNodePtr Source = GetChildAt(SourceChildIndex);
		SetChildAt(SourceChildIndex, nullptr);

		return Source;
	}

	FUAFAnimNodePtr FUAFTimedTransition::ReleaseTarget()
	{
		checkf(IsComplete(), TEXT("Attempting to release ownership of the transition target before it has completed."));

		FUAFAnimNodePtr Target = GetChildAt(TargetChildIndex);
		SetChildAt(TargetChildIndex, nullptr);

		return Target;
	}

	void FUAFTimedTransition::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
	{
		SCOPED_NAMED_EVENT(AnimNode_Update_UAFBasocTransitionNode, FColor::Blue);

		if (!IsComplete())
		{
			const float CurrentWeight = GetTotalWeight();

			TimeRemaining -= GraphContext.GetDeltaTime();

			if (TimeRemaining <= 0.0f)
			{
				// Transition is done
				TimeRemaining = 0.0f;

				// Prune our source and stop blending
				SetChildAt(SourceChildIndex, nullptr);
				SetPostAnimOp(nullptr);

				if (FUAFAnimNode* ParentNode = GetParent())
				{
					// if our parent implements IUAFTransitionContainerNode notify it that we can be pruned
					if (IUAFTransitionContainerNode* ParentContainer = ParentNode->GetInterface<IUAFTransitionContainerNode>())
					{
						ParentContainer->NotifyTransitionComplete(*this);
					}
				}
			}
			else
			{
				// Nested transition has notified us that it is complete
				if (bNestedTransitionComplete)
				{
					if (FUAFAnimNode* SourceChild = GetChildAt(SourceChildIndex))
					{
						// If we have a nested transition as our source, prune it when it completes
						if (IUAFTransitionNode* NestedTransition = SourceChild->GetInterface<IUAFTransitionNode>())
						{
							if (NestedTransition->IsComplete())
							{
								FUAFAnimNodePtr NewSource = NestedTransition->Prune();
								SetChildAt(SourceChildIndex, NewSource);
							}
						}
					}
					bNestedTransitionComplete = false;
				}

				const float SourceWeight = TimeRemaining / Duration;
				const float TargetWeight = 1.0f - SourceWeight;

				UAF_TRACE_ANIMNODE_VALUE(GraphContext, this, "TransitionTargetWeight", TargetWeight);

				GetChildAt(SourceChildIndex)->SetTotalWeight(CurrentWeight * SourceWeight);
				GetChildAt(TargetChildIndex)->SetTotalWeight(CurrentWeight * TargetWeight);
			}
		}
		else
		{
			// verify that the post anim op was cleared when we became complete
			ensure(GetPostAnimOp() == nullptr);
			
			if (FUAFAnimNode* ParentNode = GetParent())
			{
				// if we are here, we've already notified our parent that we can be pruned, but have not been pruned.
				// notify again, as this can happen if this node was re-parented before it's parent had a chance to prune it.
                			
				if (IUAFTransitionContainerNode* ParentContainer = ParentNode->GetInterface<IUAFTransitionContainerNode>())
				{
					ParentContainer->NotifyTransitionComplete(*this);
				}
			}
		}
	}
	
	void* FUAFTimedTransition::GetInterface(FUAFAnimNodeInterfaceId Id)
	{
		if (Id == IUAFTransitionNode::InterfaceId)
		{
			return static_cast<IUAFTransitionNode*>(this);
		}
		if (Id == IUAFTransitionContainerNode::InterfaceId)
		{
			return static_cast<IUAFTransitionContainerNode*>(this);
		} 

		if (const FUAFAnimNodePtr& Target = GetChildAt(TargetChildIndex))
		{
			return Target->GetInterface(Id);
		}

		return nullptr;
	}
	
#if UAF_TRACE_ENABLED
	FString FUAFTimedTransition::GetDebugName() const
	{
		static FString TimedTransitionName("Timed Transition");
		return TimedTransitionName;
	}

	UStruct* FUAFTimedTransition::GetDebugStruct() const
	{
		return FUAFTransitionNodeData::StaticStruct();
	}
#endif
	
}