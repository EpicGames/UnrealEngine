// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOpCore/UAFAnimOp.h"

namespace UE::UAF
{
	FUAFAnimOp::FUAFAnimOp(uint8 InNumInputs)
		: NumInputs(InNumInputs)
	{
	}

	FUAFAnimOp::~FUAFAnimOp()
	{
#if DO_CHECK
		checkf(bIsInitialized, TEXT("Attempting to destroy an anim op that was not initialized. Did you forget to call InitializeAs?"));
#endif
	}

	void FUAFAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		// Don't inline to ensure that we have a unique branch target for derived types that don't override this
	}

	void FUAFAnimOp::EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator)
	{
		// Don't inline to ensure that we have a unique branch target for derived types that don't override this
	}

	void FUAFAnimOp::EvaluateSynchronization(FUAFAnimOpSyncEvaluator& Evaluator)
	{
		// Don't inline to ensure that we have a unique branch target for derived types that don't override this
	}
}
