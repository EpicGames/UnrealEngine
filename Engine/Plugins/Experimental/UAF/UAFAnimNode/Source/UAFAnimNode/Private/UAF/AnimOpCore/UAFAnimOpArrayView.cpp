// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOpCore/UAFAnimOpArrayView.h"

#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"

namespace UE::UAF
{
	void FUAFAnimOpArrayView::GetAnimOps(TAdderRef<const FUAFAnimOp*> OutAnimOps) const
	{
		for (const FUAFAnimOp* AnimOp : AnimOps)
		{
			OutAnimOps.Add(AnimOp);
		}
	}

	void FUAFAnimOpArrayView::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		TUAFStack<FPoseValueBundleCoWRef>& EvaluationStack = Evaluator.GetEvaluationStack();

		for (FUAFAnimOp* AnimOp : AnimOps)
		{
			if (EvaluationStack.Num() < AnimOp->GetNumInputs()) [[unlikely]]
			{
				UE_LOGF(LogAnimation, Warning, "Too few inputs provided, AnimOp will be skipped.");
				continue;
			}

			AnimOp->EvaluateValues(Evaluator);
		}
	}

	void FUAFAnimOpArrayView::EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator)
	{
		for (FUAFAnimOp* AnimOp : AnimOps)
		{
			if (!AnimOp->HasEvaluateNotifies()) [[likely]]
			{
				// We skip AnimOps that don't implement EvaluateNotifies since most of them don't
				continue;
			}

			AnimOp->EvaluateNotifies(Evaluator);
		}
	}

	void FUAFAnimOpArrayView::EvaluateSynchronization(FUAFAnimOpSyncEvaluator& Evaluator)
	{
		for (FUAFAnimOp* AnimOp : AnimOps)
		{
			AnimOp->EvaluateSynchronization(Evaluator);
		}
	}
}
