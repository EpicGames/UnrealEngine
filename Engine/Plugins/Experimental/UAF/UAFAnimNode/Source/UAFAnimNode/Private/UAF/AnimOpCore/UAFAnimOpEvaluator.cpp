// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOpCore/UAFAnimOpEvaluator.h"

namespace UE::UAF
{
	FUAFAnimOpEvaluator::FUAFAnimOpEvaluator() = default;

	FUAFAnimOpEvaluator::~FUAFAnimOpEvaluator() = default;
}
