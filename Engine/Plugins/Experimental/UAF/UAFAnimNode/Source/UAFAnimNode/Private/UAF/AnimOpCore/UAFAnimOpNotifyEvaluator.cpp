// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOpCore/UAFAnimOpNotifyEvaluator.h"

#include "Animation/AnimNotifyQueue.h"

namespace UE::UAF
{
	FUAFAnimOpNotifyEvaluator::FUAFAnimOpNotifyEvaluator() = default;

	FUAFAnimOpNotifyEvaluator::~FUAFAnimOpNotifyEvaluator() = default;
}
