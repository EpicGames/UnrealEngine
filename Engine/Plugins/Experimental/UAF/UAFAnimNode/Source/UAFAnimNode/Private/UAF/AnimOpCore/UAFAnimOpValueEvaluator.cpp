// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/Attributes/AttributeBindingDataCache.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"

namespace UE::UAF
{
	const FUAFStackName VALUE_STACK_NAME = FName("Value Stack");

	FUAFAnimOpValueEvaluationContext::FUAFAnimOpValueEvaluationContext() = default;

	FUAFAnimOpValueEvaluationContext::FUAFAnimOpValueEvaluationContext(TNonNullPtr<const UAbstractSkeletonSetBinding> InSetBinding, const USkeletalMesh* InSkeletalMesh, FName InSetName)
	{
		if (const FAttributeBindingDataPtr& InBindingData = GAttributeBindingDataCache.GetOrAdd(InSetBinding, InSkeletalMesh))
		{
			if (FAttributeNamedSetPtr InNamedSet = InBindingData->FindNamedSet(InSetName))
			{
				SetBinding = InSetBinding;
				SkeletalMesh = InSkeletalMesh;
				Skeleton = InSetBinding->GetSkeleton();
				BindingData = InBindingData;
				NamedSet = MoveTemp(InNamedSet);
				DefaultAttributeValues = NamedSet->GetDefaultAttributeValues();
			}
		}
	}

	bool FUAFAnimOpValueEvaluationContext::SetNamedSet(FName InSetName)
	{
		if (BindingData)
		{
			if (FAttributeNamedSetPtr InNamedSet = BindingData->FindNamedSet(InSetName))
			{
				NamedSet = MoveTemp(InNamedSet);
				DefaultAttributeValues = NamedSet->GetDefaultAttributeValues();
				return true;
			}
		}

		// Named set not found
		return false;
	}

	FUAFAnimOpValueEvaluator::FUAFAnimOpValueEvaluator(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName, int32 InCurrentLOD)
		: EvaluationStack(VALUE_STACK_NAME)
		, ValueTransformerMap(FValueRuntimeRegistry::Get().GetTransformerMap())
		, CurrentLOD(InCurrentLOD)
	{
		EvaluationContextStack.Push(FUAFAnimOpValueEvaluationContext(SetBinding, SkeletalMesh, SetName));
		checkf(EvaluationContextStack.Last().IsValid(), TEXT("Pushed an invalid AnimOp value evaluation context."));
	}

	const FUAFAnimOpValueEvaluationContext& FUAFAnimOpValueEvaluator::GetActiveEvaluationContext() const
	{
		return EvaluationContextStack.Last();
	}

	const FAttributeNamedSetPtr& FUAFAnimOpValueEvaluator::GetActiveNamedSet() const
	{
		return EvaluationContextStack.Last().GetNamedSet();
	}

	bool FUAFAnimOpValueEvaluator::PushEvaluationContext(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName)
	{
		FUAFAnimOpValueEvaluationContext NewContext(SetBinding, SkeletalMesh, SetName);
		if (NewContext.IsValid())
		{
			EvaluationContextStack.Push(MoveTemp(NewContext));
			checkf(EvaluationContextStack.Last().IsValid(), TEXT("Pushed an invalid AnimOp value evaluation context."));
			return true;
		}

		// New evaluation context isn't valid
		return false;
	}

	bool FUAFAnimOpValueEvaluator::PushEvaluationContext(FName SetName)
	{
		// Copy the currently active context
		FUAFAnimOpValueEvaluationContext NewContext = EvaluationContextStack.Last();
		if (NewContext.SetNamedSet(SetName))
		{
			EvaluationContextStack.Push(MoveTemp(NewContext));
			checkf(EvaluationContextStack.Last().IsValid(), TEXT("Pushed an invalid AnimOp value evaluation context."));
			return true;
		}

		// Failed to find the named set in our currently active binding
		return false;
	}

	bool FUAFAnimOpValueEvaluator::PopEvaluationContext()
	{
		if (EvaluationContextStack.Num() > 1)
		{
			EvaluationContextStack.Pop(EAllowShrinking::No);
			return true;
		}

		// Cannot pop an entry from an empty stack or the last entry
		return false;
	}
}
