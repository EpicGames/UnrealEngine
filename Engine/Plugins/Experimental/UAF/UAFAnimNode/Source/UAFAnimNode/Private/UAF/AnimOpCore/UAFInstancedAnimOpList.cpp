// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOpCore/UAFInstancedAnimOpList.h"

#include "UAF/AnimOpCore/UAFAnimOp.h"
#include "UAF/AnimOpCore/UAFAnimOpListBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFInstancedAnimOpList)

namespace UE::UAF
{
	namespace Private
	{
		static void CollectAnimOps(const TArrayView<const FUAFAnimOp* const>& AnimOps, FUAFInstancedAnimOpList& OutAnimOps)
		{
			for (const FUAFAnimOp* AnimOp : AnimOps)
			{
				const UScriptStruct* AnimOpType = AnimOp->GetStruct();
				if (AnimOpType == FUAFAnimOpListBase::StaticStruct())
				{
					const FUAFAnimOpListBase* AnimOpList = static_cast<const FUAFAnimOpListBase*>(AnimOp);

					TArray<const FUAFAnimOp*> NestedAnimOps;
					AnimOpList->GetAnimOps(NestedAnimOps);

					CollectAnimOps(NestedAnimOps, OutAnimOps);
				}
				else
				{
					OutAnimOps.AnimOps.Add(FInstancedStruct(AnimOpType));
					AnimOpType->CopyScriptStruct(OutAnimOps.AnimOps.Last().GetMutableMemory(), AnimOp);
				}
			}
		}
	}

	FUAFInstancedAnimOpList::FUAFInstancedAnimOpList(const TArray<FUAFAnimOp*>& InAnimOps)
	{
		Private::CollectAnimOps(MakeConstArrayView(InAnimOps.GetData(), InAnimOps.Num()), *this);
	}
}
