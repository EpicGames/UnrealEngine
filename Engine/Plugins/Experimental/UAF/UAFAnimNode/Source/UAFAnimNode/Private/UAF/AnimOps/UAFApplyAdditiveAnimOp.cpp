// Copyright Epic Games, Inc. All Rights Reserved.


#include "UAF/AnimOps/UAFApplyAdditiveAnimOp.h"

#include "Animation/SkeletonRemappingRegistry.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/BoneSpace.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"

namespace UE::UAF
{
	FUAFApplyAdditiveAnimOp::FUAFApplyAdditiveAnimOp()
		: FUAFAnimOp(2) 
	{
		InitializeAs<FUAFApplyAdditiveAnimOp>();
	}

	void FUAFApplyAdditiveAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

		// Pop our inputs - the additive being on top and base the next underneath 
		FPoseValueBundleCoWRef AdditiveInput = Evaluator.GetEvaluationStack().Pop();
		FPoseValueBundleCoWRef BaseInput = Evaluator.GetEvaluationStack().Pop();

		const FPoseValueBundle& Additive = AdditiveInput.Get();
		FPoseValueBundle& Base = BaseInput.GetMutable();
		
		// If the additive is not relevant or empty, we simply fall back to the base pose
		// Both scenarios are valid and do not necessarily indicate a setup error so we will not raise a waring 
		if (!FAnimWeight::IsRelevant(AdditiveWeight) || Additive.IsEmpty())
		{
			Evaluator.GetEvaluationStack().Push(MoveTemp(BaseInput));
			return;
		}

		// If the additive animation is not empty, but it also is not an additive warn the user and fall back to the base pose instead
		if (!Additive.IsAdditive())
		{
			Evaluator.GetEvaluationStack().Push(MoveTemp(BaseInput));
			UE_LOGF(LogAnimation, Error, "FUAFApplyAdditiveAnimOp::EvaluateValues - The provided additive animation is not an additive");
			return;
		}
		
		// If the base is empty init it with default values 
		if (Base.IsEmpty())
		{
			check(BaseInput.IsMutable());
			BaseInput.GetMutable().InitWithValueSpace(FValueSpace(EValueSpaceType::Local));
			
			UE_LOGF(LogAnimation, Warning, "FUAFApplyAdditiveAnimOp::EvaluateValues - The base input is empty. Will use a ref pose instead.")
		}
		
		// Named sets must match as it ensures that our inputs have the same sizes/shapes
		check(Base.GetNamedSet() == Additive.GetNamedSet());
		
		// If our additive input is in mesh rotation space, we ensure our base is in that space as well
		bool bHasToConvertBaseToMeshSpace = false;
		if (Additive.GetValueSpace().GetMixedSpaceFlags() == EMixedSpaceFlags::MeshRotation)
		{
			const FValueSpace BaseSpace = Base.GetValueSpace();
			bHasToConvertBaseToMeshSpace = BaseSpace.GetMixedSpaceFlags() != EMixedSpaceFlags::MeshRotation;
		}

		// Modify our base input in-place
		if (bHasToConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::LocalToMeshRotation(Base, Base);
		}

		if (OptionalBlendMask)
		{
			FValueBundleStack PerValueWeights(Evaluator.GetActiveNamedSet());
			BuildPerValueWeights(Evaluator, Base, PerValueWeights);
			Transformers::FApplyAdditiveSpace::Apply(Evaluator.GetTransformerMap(), Base, Additive, PerValueWeights, AdditiveWeight, Base);
		}
		else
		{
			Transformers::FApplyAdditiveSpace::Apply(Evaluator.GetTransformerMap(), Base, Additive, AdditiveWeight, Base);
		}
		
		if (bHasToConvertBaseToMeshSpace)
		{
			Transformers::FBoneSpace::MeshRotationToLocal(Base, Base);
		}

		Transformers::FSanitize::Apply(Evaluator.GetTransformerMap(), Base);
		Evaluator.GetEvaluationStack().Push(MoveTemp(BaseInput));
	}

	void FUAFApplyAdditiveAnimOp::SetAdditiveWeight(const float InWeight)
	{
		AdditiveWeight = InWeight;
	}

	void FUAFApplyAdditiveAnimOp::SetBlendMask(const TObjectPtr<UUAFBlendMask> InBlendMask)
	{
		OptionalBlendMask = InBlendMask;
	}

	void FUAFApplyAdditiveAnimOp::BuildPerValueWeights(const FUAFAnimOpValueEvaluator& Evaluator, const FPoseValueBundle& BaseInput, FValueBundleStack& OutPerValueWeights)
	{
		if (OptionalBlendMask == nullptr)
		{
			return;
		}
		
		UScriptStruct* FloatAttributeType = FFloatAnimationAttribute::StaticStruct();
		const TObjectPtr<USkeleton> SourceSkeleton = OptionalBlendMask->GetSkeleton();
		const UUAFBlendMask::FSkeletonBoneWeightArray& BoneMaskWeights = OptionalBlendMask->GetSkeletonBoneWeights(); 
		const UUAFBlendMask::FSkeletonCurveWeightArray& CurveMaskWeights = OptionalBlendMask->GetSkeletonCurveWeights(); 
		const UUAFBlendMask::FSkeletonAttributeWeightArray& AttributeMaskWeights = OptionalBlendMask->GetSkeletonAttributeWeights();
		
		// Build our per-value weights (only for bound values)
		FBoundMapCollection& PerValueWeightMaps = OutPerValueWeights.GetBoundValueMaps();
		for (auto It = BaseInput.GetBoundValueMaps().CreateConstIterator(); It; ++It)
		{
			UScriptStruct* AttributeType = It.GetAttributeType();
			// Every attribute type maps to a float - the weight we want to apply per attribute
			FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo(AttributeType, FloatAttributeType);
			// Add the float attribute to our per value weight map
			TBoundValueMap<FFloatAnimationAttribute>* SetWeights = PerValueWeightMaps.Add<FFloatAnimationAttribute>(MappingKey);
			
			// Bone transforms weights
			if (const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneMap = Cast<FBoneTransformAnimationAttribute>(It.GetMap()))
			{
				const FAttributeTypedSetPtr& BoneSet = BoneMap->GetTypedSet();
				const int32 NumBonesInSet = BoneMap->Num();

				check(OptionalBlendMask->GetSkeletonBoneWeights().Num() >= NumBonesInSet);
				
				const USkeleton* TargetSkeleton = Evaluator.GetActiveEvaluationContext().GetSkeleton();
				const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
				
				for (FAttributeSetIndex SetIndex(0); SetIndex < NumBonesInSet; ++SetIndex)
				{
					const FAttributeBindingIndex BindingIndex = BoneSet->GetBindingIndex(SetIndex);
					const FSkeletonPoseBoneIndex TargetSkeletonBoneIndex = BindingIndex;
					int32 WeightIndex = TargetSkeletonBoneIndex.GetInt();
					if (SkeletonRemapping.IsValid())
					{
						const FSkeletonPoseBoneIndex SourceSkeletonBoneIndex(SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex.GetInt()));
						WeightIndex = SourceSkeletonBoneIndex.IsValid() ? SourceSkeletonBoneIndex.GetInt() : INDEX_NONE;
					}
					
					(*SetWeights)[SetIndex].Value = WeightIndex != INDEX_NONE ? BoneMaskWeights[WeightIndex] : 0.0f;
					(*SetWeights)[SetIndex].Value *= AdditiveWeight;
				}
			}
			// Curve and attribute weights
			else if (const TBoundValueMap<FFloatAnimationAttribute>* FloatMap = Cast<FFloatAnimationAttribute>(It.GetMap()))
			{
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 0.0f });
				
				// Curves
				const FAttributeTypedSetPtr& FloatSet = FloatMap->GetTypedSet();
				CurveMaskWeights.ForEachElement([&FloatSet, SetWeights, ActiveAdditiveWeight = AdditiveWeight](const UE::Anim::FCurveElement& Curve)
					{
						const FAttributeSetIndex SetIndex = FloatSet->FindIndex(Curve.Name);
						if (SetIndex.IsValid())
						{
							(*SetWeights)[SetIndex].Value = Curve.Value;
							(*SetWeights)[SetIndex].Value *= ActiveAdditiveWeight;
						}
					});
				
				// Attributes
				for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttributeWeight : AttributeMaskWeights)
				{
					const FAttributeSetIndex SetIndex = FloatSet->FindIndex(AttributeWeight.Attribute.GetName());
					if (SetIndex.IsValid())
					{
						(*SetWeights)[SetIndex].Value = AttributeWeight.Weight;
						(*SetWeights)[SetIndex].Value *= AdditiveWeight;
					}
				}
			}
			else
			{
				SetWeights->FillWithValue(FFloatAnimationAttribute{ 0.0f });

				const FAttributeTypedSetPtr& TypedSet = It.GetMap()->GetTypedSet();
				for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& AttributeWeight : AttributeMaskWeights)
				{
					const FAttributeSetIndex SetIndex = TypedSet->FindIndex(AttributeWeight.Attribute.GetName());
					if (SetIndex.IsValid())
					{
						(*SetWeights)[SetIndex].Value = AttributeWeight.Weight;
						(*SetWeights)[SetIndex].Value *= AdditiveWeight;
					}
				}
			}
		}
	}
}
