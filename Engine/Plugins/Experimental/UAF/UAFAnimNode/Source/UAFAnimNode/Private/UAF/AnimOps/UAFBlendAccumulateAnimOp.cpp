// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFBlendAccumulateAnimOp.h"

#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"

namespace UE::UAF
{
	FUAFBlendAccumulateAnimOp::FUAFBlendAccumulateAnimOp()
		: FUAFAnimOp(2)
	{
		InitializeAs<FUAFBlendAccumulateAnimOp>();
	}

	void FUAFBlendAccumulateAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		// Pop our inputs
		FPoseValueBundleCoWRef InputRefB = Evaluator.GetEvaluationStack().Pop();
		FPoseValueBundleCoWRef InputRefA = Evaluator.GetEvaluationStack().Pop();

		const FPoseValueBundle& InputA = InputRefA.Get();
		const FPoseValueBundle& InputB = InputRefB.Get();

		// Overwrite/accumulate cannot be used with empty inputs because we cannot tell
		// which space they are supposed to be in until we hit an accumulate but by then
		// the overwrite weight has been discarded and the remaining weights won't sum to 1.0
		// To avoid doing a partial blend with an unknown pose, we return an empty result
		// and let a parent AnimOp handle it
		if (InputA.IsEmpty())
		{
			Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefA));
		}
		else if (InputB.IsEmpty())
		{
			Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefB));
		}
		else
		{
			// Named sets must match as it ensures that our inputs have the same sizes/shapes
			check(InputA.GetNamedSet() == InputB.GetNamedSet());

			FPoseValueBundleCoWRef& OutputRef = FindMutableCoWRef(InputRefB, InputRefA);

			Transformers::FAccumulate::Apply(Evaluator.GetTransformerMap(), InputB, InputA, ScaleFactor, OutputRef.GetMutable());

			Evaluator.GetEvaluationStack().Push(MoveTemp(OutputRef));
		}
	}
}
