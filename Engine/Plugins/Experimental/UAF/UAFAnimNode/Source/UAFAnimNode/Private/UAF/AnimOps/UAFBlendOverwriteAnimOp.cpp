// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFBlendOverwriteAnimOp.h"

#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"

namespace UE::UAF
{
	FUAFBlendOverwriteAnimOp::FUAFBlendOverwriteAnimOp()
		: FUAFAnimOp(1)
	{
		InitializeAs<FUAFBlendOverwriteAnimOp>();
	}

	void FUAFBlendOverwriteAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		FPoseValueBundleCoWRef* InputRef = Evaluator.GetEvaluationStack().PeekMutable();
		FPoseValueBundle& Input = InputRef->GetMutable();

		// We cannot scale an empty input, let the next AnimOp handle it
		if (!Input.IsEmpty())
		{
			// Overwrite in place
			Transformers::FOverwrite::Apply(Evaluator.GetTransformerMap(), Input, ScaleFactor, Input);
		}
	}
}
