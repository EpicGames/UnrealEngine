// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFBlendTwoAnimOp.h"

#include "Animation/InputScaleBias.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"
#include "UAF/ValueRuntime/ValueBundle.h"

namespace UE::UAF
{
	FUAFBlendTwoAnimOp::FUAFBlendTwoAnimOp()
		: FUAFAnimOp(2)
	{
		InitializeAs<FUAFBlendTwoAnimOp>();
	}

	void FUAFBlendTwoAnimOp::SetInterpolationAlpha(float InInterpolationAlpha)
	{
		DesiredInterpolationAlpha = InInterpolationAlpha;
	}

	void FUAFBlendTwoAnimOp::SetInterpolationCurve(const FName& InAlphaSourceCurveName, const int8 InAlphaCurveInputIndex, TFunction<float(float)> InInputScaleBiasClampFn)
	{
		AlphaSourceCurveName = InAlphaSourceCurveName;
		AlphaCurveInputIndex = InAlphaCurveInputIndex;
		InputScaleBiasClampFn = MoveTemp(InInputScaleBiasClampFn);
	}

	void FUAFBlendTwoAnimOp::SetBlendOption(EAlphaBlendOption InBlendOption)
	{
		BlendOption = InBlendOption;
	}
	
	void FUAFBlendTwoAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

		// Pop our inputs
		FPoseValueBundleCoWRef InputRefB = Evaluator.GetEvaluationStack().Pop();
		FPoseValueBundleCoWRef InputRefA = Evaluator.GetEvaluationStack().Pop();

		const FPoseValueBundle& InputA = InputRefA.Get();
		const FPoseValueBundle& InputB = InputRefB.Get();

		const float WeightOfPoseB = GetInterpolationAlpha(InputA, InputB);
		ActualInterpolationAlpha = WeightOfPoseB;

		if (!FAnimWeight::IsRelevant(WeightOfPoseB))
		{
			// Input A has full weight, push it back even if it might be empty
			Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefA));
		}
		else if (FAnimWeight::IsFullWeight(WeightOfPoseB))
		{
			// Input B has full weight, push it back even if it might be empty
			Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefB));
		}
		else
		{
			// TODO: It would be nice if we could use an immutable reference to the bind pose to avoid copying it

			if (InputA.IsEmpty())
			{
				if (InputB.IsEmpty())
				{
					// Both inputs are empty, return an empty output
					Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefB));
					return;
				}
				else
				{
					// A is empty, initialize it with the same space as B
					check(InputRefA.IsMutable());
					InputRefA.GetMutable().InitWithValueSpace(InputB.GetValueSpace());
				}
			}
			else
			{
				if (InputB.IsEmpty())
				{
					// B is empty, initialize it with the same space as A
					check(InputRefB.IsMutable());
					InputRefB.GetMutable().InitWithValueSpace(InputA.GetValueSpace());
				}
			}

			// Named sets must match as it ensures that our inputs have the same sizes/shapes
			check(InputA.GetNamedSet() == InputB.GetNamedSet());

			FPoseValueBundleCoWRef& OutputRef = FindMutableCoWRef(InputRefB, InputRefA);

			Transformers::FInterpolate::Apply(Evaluator.GetTransformerMap(), InputA, InputB, WeightOfPoseB, OutputRef.GetMutable());
			Transformers::FSanitize::Apply(Evaluator.GetTransformerMap(), OutputRef.GetMutable());

			Evaluator.GetEvaluationStack().Push(MoveTemp(OutputRef));
		}
	}

	float FUAFBlendTwoAnimOp::GetInterpolationAlpha(const FPoseValueBundle& InputA, const FPoseValueBundle& InputB) const
	{
		float Alpha = DesiredInterpolationAlpha;

		if (AlphaSourceCurveName != NAME_None && AlphaCurveInputIndex != INDEX_NONE)
		{
			const FPoseValueBundle& AlphaCurveCollection = AlphaCurveInputIndex == 0 ? InputA : InputB;
			if (const TBoundValueMap<FFloatAnimationAttribute>* FloatMap = AlphaCurveCollection.FindFloatCurves())
			{
				const FAttributeTypedSetPtr& TypedSet = FloatMap->GetTypedSet();
				if (FAttributeSetIndex CurveIndex = TypedSet->FindIndex(AlphaSourceCurveName))
				{
					Alpha = (*FloatMap)[CurveIndex].Value;
				}
				else
				{
					// Curve name wasn't found
					Alpha = 0.0f;
				}
			}
			else
			{
				// No float curves found
				Alpha = 0.0f;
			}

			if (InputScaleBiasClampFn.IsSet())
			{
				Alpha = InputScaleBiasClampFn(Alpha);
			}
		}

		Alpha = FAlphaBlend::AlphaToBlendOption(Alpha, BlendOption);

		return FMath::Clamp(Alpha, 0.0f, 1.0f);
	}
}
