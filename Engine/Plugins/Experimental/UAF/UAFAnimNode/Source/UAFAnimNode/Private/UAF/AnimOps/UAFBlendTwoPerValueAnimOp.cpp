// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFBlendTwoPerValueAnimOp.h"

#include "Animation/InputScaleBias.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/BlendProfile/UAFBlendProfile.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"
#include "UAF/ValueRuntime/ValueBundle.h"

namespace UE::UAF
{
	FUAFBlendTwoPerValueAnimOp::FUAFBlendTwoPerValueAnimOp()
		: FUAFAnimOp(2)
		, BlendProfile(nullptr)
	{
		InitializeAs<FUAFBlendTwoPerValueAnimOp>();
	}
	
	FUAFBlendTwoPerValueAnimOp::FUAFBlendTwoPerValueAnimOp(TNonNullPtr<UUAFBlendProfile> InBlendProfile, EAlphaBlendOption InBlendOption)
		: FUAFAnimOp(2)
		, BlendProfile(InBlendProfile)
		, BlendOption(InBlendOption)
	{
		InitializeAs<FUAFBlendTwoPerValueAnimOp>();
	}

	void FUAFBlendTwoPerValueAnimOp::SetLinearWeightB(float InInterpolationAlpha)
	{
		LinearGlobalWeightB = FMath::Clamp(InInterpolationAlpha, 0.0f, 1.0f);
	}

	void FUAFBlendTwoPerValueAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

		// Pop our inputs
		FPoseValueBundleCoWRef InputRefB = Evaluator.GetEvaluationStack().Pop();
		FPoseValueBundleCoWRef InputRefA = Evaluator.GetEvaluationStack().Pop();

		const FPoseValueBundle& InputA = InputRefA.Get();
		const FPoseValueBundle& InputB = InputRefB.Get();

		const float GlobalWeightB = FAlphaBlend::AlphaToBlendOption(LinearGlobalWeightB, BlendOption);
		const float GlobalWeightA = 1.0 - GlobalWeightB;

		if (!FAnimWeight::IsRelevant(GlobalWeightB))
		{
			// Input A has full weight, push it back even if it might be empty
			Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefA));
		}
		else if (FAnimWeight::IsFullWeight(GlobalWeightB))
		{
			// Input B has full weight, push it back even if it might be empty
			Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefB));
		}
		else
		{
			// TODO: It would be nice if we could use an immutable reference to the bind pose to avoid copying it

			if (InputA.IsEmpty())
			{
				if (InputB.IsEmpty())
				{
					// Both inputs are empty, return an empty output
					Evaluator.GetEvaluationStack().Push(MoveTemp(InputRefB));
					return;
				}
				else
				{
					// A is empty, initialize it with the same space as B
					check(InputRefA.IsMutable());
					InputRefA.GetMutable().InitWithValueSpace(InputB.GetValueSpace());
				}
			}
			else
			{
				if (InputB.IsEmpty())
				{
					// B is empty, initialize it with the same space as A
					check(InputRefB.IsMutable());
					InputRefB.GetMutable().InitWithValueSpace(InputA.GetValueSpace());
				}
			}

			// Named sets must match as it ensures that our inputs have the same sizes/shapes
			check(InputA.GetNamedSet() == InputB.GetNamedSet());

			FPoseValueBundleCoWRef& OutputRef = FindMutableCoWRef(InputRefB, InputRefA);

			FValueBundleStack PerValueWeights(Evaluator.GetActiveNamedSet());
			PerValueWeights.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));
			{
				FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute, FFloatAnimationAttribute>();
				TBoundValueMap<FFloatAnimationAttribute>* BoneWeightsMap = PerValueWeights.GetBoundValueMaps().Add<FFloatAnimationAttribute>(MappingKey);

				// TODO: In the future we don't want to be using USkeleton backed blend profiles and instead should use blend profiles authored
				// for the abstract skeleton but in the meantime we can do this shim even though its a bit slow
				const UUAFBlendProfile::FSkeletonBoneWeightArray& ProfileBoneWeights = BlendProfile->GetSkeletonBoneWeights();
				BoneWeightsMap->FillWithValue(FFloatAnimationAttribute{ GlobalWeightB });

				const FAttributeTypedSetPtr& BoneTransforms = InputA.GetNamedSet()->FindTypedSet<FBoneTransformAnimationAttribute>();
				const FReferenceSkeleton& RefSkeleton = BlendProfile->GetSkeleton()->GetReferenceSkeleton();
				
				if (BlendProfile->GetType() == EUAFBlendProfileType::TimeFactor)
				{
					// For time profiles we scale the global weight by the profile value
					// and then sample the blend curve.
					for (int32 BoneIndex = 0; BoneIndex < ProfileBoneWeights.Num(); ++BoneIndex)
					{
						const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);

						FAttributeSetKey Key = BoneTransforms->FindKey(BoneName); // O(log n)
						if (Key.IsValid())
						{
							const float ProfileValue = ProfileBoneWeights[BoneIndex];
							check(ProfileValue >= 0.0f && ProfileValue <= 1.0f);
							
							const float BoneLinearWeightB = ProfileValue != 0.0f
								? FMath::Clamp(LinearGlobalWeightB / ProfileValue, 0.0f, 1.0f)
								: 1.0f;
							
							const float BoneWeightB = FAlphaBlend::AlphaToBlendOption(BoneLinearWeightB, BlendOption); // Convert linear weight to curve weight

							FAttributeSetIndex Index = BoneTransforms->FindIndex(Key); // O(log n)
							BoneWeightsMap->SetValue(Index, FFloatAnimationAttribute{ BoneWeightB });
						}
					}
				}
				else
				{
					// For weight profiles we sample the blend curve using the global weight and then
					// scale it by the profile value in both directions and normalize the result.
					for (int32 BoneIndex = 0; BoneIndex < ProfileBoneWeights.Num(); ++BoneIndex)
					{
						const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);

						FAttributeSetKey Key = BoneTransforms->FindKey(BoneName); // O(log n)
						if (Key.IsValid())
						{
							const float ProfileValue = ProfileBoneWeights[BoneIndex];
							check(ProfileValue >= 1.0f);
							
							float BoneWeightA = GlobalWeightA / ProfileValue;
							float BoneWeightB = GlobalWeightB * ProfileValue;

							const float SumBoneWeights = BoneWeightA + BoneWeightB;
							BoneWeightB /= SumBoneWeights;

							FAttributeSetIndex Index = BoneTransforms->FindIndex(Key); // O(log n)
							BoneWeightsMap->SetValue(Index, FFloatAnimationAttribute{ BoneWeightB });
						}
					}
				}
			}
			
			Transformers::FInterpolate::Apply(Evaluator.GetTransformerMap(), InputA, InputB, PerValueWeights, LinearGlobalWeightB, OutputRef.GetMutable());
			Transformers::FSanitize::Apply(Evaluator.GetTransformerMap(), OutputRef.GetMutable());

			Evaluator.GetEvaluationStack().Push(MoveTemp(OutputRef));
		}
	}
}