// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFDecompressAnimSequenceAnimOp.h"

#include "Animation/AnimSequence.h"
#include "Animation/AnimRootMotionProvider.h"
#include "EvaluationVM/EvaluationVM.h"	// for FEvaluationTaskContext
#include "EvaluationVM/Tasks/DecompressionTools.h"
#include "UAF/AnimOpCore/UAFAnimOpNotifyEvaluator.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"

namespace UE::UAF
{
	FUAFDecompressAnimSequenceAnimOp::FUAFDecompressAnimSequenceAnimOp()
		: FUAFAnimOp(0)
	{
		InitializeAs<FUAFDecompressAnimSequenceAnimOp>();
	}

	void FUAFDecompressAnimSequenceAnimOp::Initialize(TObjectPtr<const UAnimSequence> InAnimSequence, float StartTime, bool bInIsLooping, bool bInInterpolate, bool bInExtractTrajectory)
	{
		AnimSequence = InAnimSequence;
		CurrentTime = StartTime;
		PreviousTime = StartTime;
		Duration = InAnimSequence ? InAnimSequence->GetPlayLength() : 0.0f;
		bIsLooping = bInIsLooping;
		bInterpolate = bInInterpolate;
		bExtractTrajectory = bInExtractTrajectory;
	}

	ETypeAdvanceAnim FUAFDecompressAnimSequenceAnimOp::AdvanceTime(float DeltaTime)
	{
		if (AnimSequence)
		{
			PreviousTime = CurrentTime;

			DeltaTimeRecord.Set(PreviousTime, DeltaTime);
			TimeAdvanceResult = FAnimationRuntime::AdvanceTime(bIsLooping, DeltaTime, CurrentTime, Duration);
		}

		return TimeAdvanceResult;
	}

	void FUAFDecompressAnimSequenceAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

		FPoseValueBundleStack ValueBundle(AnimOpEvaluationContext.GetNamedSet());

		const UAnimSequence* AnimSequencePtr = AnimSequence.Get();
		if (AnimSequencePtr && AnimSequencePtr->GetSkeleton())
		{
			const bool bExtractRootMotion = bExtractTrajectory;
			const FAnimExtractContext ExtractionContext((double)CurrentTime, bExtractRootMotion, DeltaTimeRecord, bIsLooping);

			const bool bIsAdditive = AnimSequencePtr->IsValidAdditive();
			const EAdditiveAnimationType AdditiveType = AnimSequencePtr->GetAdditiveAnimType();
			const bool bUseRawData = FDecompressionTools::ShouldUseRawData(AnimSequencePtr, ExtractionContext);

			FValueSpace ValueSpace;
			if (bIsAdditive && AdditiveType == AAT_RotationOffsetMeshSpace)
			{
				ValueSpace = FValueSpace(EMixedSpaceFlags::MeshRotation, bIsAdditive);
			}
			else
			{
				ValueSpace = FValueSpace(EValueSpaceType::Local, bIsAdditive);
			}

			ValueBundle.InitWithValueSpace(ValueSpace);

			// Coerce for now
			FEvaluationTaskContext VMContext(AnimOpEvaluationContext.GetSetBinding(), AnimOpEvaluationContext.GetSkeletalMesh(), AnimOpEvaluationContext.GetNamedSet()->GetName());

			FDecompressionTools::GetAnimationPose(AnimSequencePtr, ExtractionContext, VMContext, ValueBundle, bUseRawData);
			FDecompressionTools::GetAnimationCurves(AnimSequencePtr, ExtractionContext, VMContext, ValueBundle, bUseRawData);
			FDecompressionTools::GetAnimationAttributes(AnimSequencePtr, ExtractionContext, VMContext, ValueBundle, bUseRawData);

			// If the sequence has root motion enabled, extract it into its own attribute
			if (AnimSequence->HasRootMotion())
			{
				FAttributeNamedSetPtr NamedSet = ValueBundle.GetNamedSet();
				if (FAttributeTypedSetPtr AttributeTypedSet = NamedSet->FindTypedSet<FTransformAnimationAttribute>())
				{
					// AttributeName: RootMotionDelta
					if (const FAttributeSetIndex SetIndex = AttributeTypedSet->FindIndex(UE::Anim::IAnimRootMotionProvider::AttributeName))
					{
						// This attribute is present within the current named set

						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FTransformAnimationAttribute>();
						TBoundValueMap<FTransformAnimationAttribute>* Map = ValueBundle.GetBoundValueMaps().Find<FTransformAnimationAttribute>(MappingKey);
						checkf(Map != nullptr, TEXT("This attribute exists within our named set but isn't present in the output value bundle"));

						const FTransform RootMotionTransform = AnimSequence->ExtractRootMotion(ExtractionContext);
						(*Map)[SetIndex].Value = RootMotionTransform;
					}
				}
			}
		}

		Evaluator.GetEvaluationStack().Push(FPoseValueBundleCoWRef::MakeFrom(MoveTemp(ValueBundle)));
	}

	void FUAFDecompressAnimSequenceAnimOp::EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator)
	{
		const UAnimSequence* AnimSequencePtr = AnimSequence.Get();
		if (AnimSequencePtr && AnimSequencePtr->GetSkeleton())
		{
			FAnimTickRecord TickRecord;
			TickRecord.TimeAccumulator = &CurrentTime;
			TickRecord.bLooping = bIsLooping;

			FAnimNotifyContext NotifyContext(TickRecord);
			AnimSequencePtr->GetAnimNotifies(PreviousTime, DeltaTimeRecord.Delta, NotifyContext);

			Evaluator.GetNotifies().Append(NotifyContext.ActiveNotifies);
		}
	}
}
