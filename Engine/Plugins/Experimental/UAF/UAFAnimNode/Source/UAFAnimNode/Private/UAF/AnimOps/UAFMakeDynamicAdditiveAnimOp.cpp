// Copyright Epic Games, Inc. All Rights Reserved.


#include "UAF/AnimOps/UAFMakeDynamicAdditiveAnimOp.h"

#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/BoneSpace.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "Logging/StructuredLog.h"

namespace UE::UAF
{
	FUAFMakeDynamicAdditiveAnimOp::FUAFMakeDynamicAdditiveAnimOp()
		: FUAFAnimOp(2)
	{
		InitializeAs<FUAFMakeDynamicAdditiveAnimOp>();
	}

	void FUAFMakeDynamicAdditiveAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		// Stack order: Base (bottom), Source (top)
		// Pop Source first (top of stack), then Base (underneath)
		FPoseValueBundleCoWRef SourceInput = Evaluator.GetEvaluationStack().Pop();
		FPoseValueBundleCoWRef BaseInput = Evaluator.GetEvaluationStack().Pop();

		const FValueSpace AdditiveSpace = bMeshSpaceAdditive
			? FValueSpace(EMixedSpaceFlags::MeshRotation, true)
			: FValueSpace(EValueSpaceType::Local, true);

		bool bInvalidSetup = false;
		if (BaseInput.Get().IsEmpty())
		{
			bInvalidSetup = true;
			UE_LOGF(LogAnimation, Error, "FUAFMakeDynamicAdditiveAnimOp - Base input is empty.");
		}
		else if (BaseInput.Get().IsAdditive())
		{
			bInvalidSetup = true;
			UE_LOGF(LogAnimation, Error, "FUAFMakeDynamicAdditiveAnimOp - Base input is additive. Expected non-additive base.");
		}

		if (SourceInput.Get().IsEmpty())
		{
			bInvalidSetup = true;
			UE_LOGF(LogAnimation, Error, "FUAFMakeDynamicAdditiveAnimOp - Source input is empty.");
		}
		else if (SourceInput.Get().IsAdditive())
		{
			bInvalidSetup = true;
			UE_LOGF(LogAnimation, Error, "FUAFMakeDynamicAdditiveAnimOp - Source input is additive. Expected non-additive Source.");
		}

		if (bInvalidSetup)
		{
			FPoseValueBundleStack Output(Evaluator.GetActiveNamedSet());
			Output.InitWithValueSpace(AdditiveSpace);
			Evaluator.GetEvaluationStack().Push(FPoseValueBundleCoWRef::MakeFrom(MoveTemp(Output)));
			return;
		}

		if (bMeshSpaceAdditive)
		{
			// Convert both inputs to mesh rotation space before computing the additive delta
			BaseInput.ForceMutable();
			SourceInput.ForceMutable();
			Transformers::FBoneSpace::LocalToMeshRotation(BaseInput.GetMutable(), BaseInput.GetMutable());
			Transformers::FBoneSpace::LocalToMeshRotation(SourceInput.GetMutable(), SourceInput.GetMutable());
		}

		FPoseValueBundleStack Output(Evaluator.GetActiveNamedSet());
		Output.InitWithValueSpace(AdditiveSpace);

		Transformers::FMakeAdditiveSpace::Apply(Evaluator.GetTransformerMap(), BaseInput.Get(), SourceInput.Get(), Output);

		Evaluator.GetEvaluationStack().Push(FPoseValueBundleCoWRef::MakeFrom(MoveTemp(Output)));
	}

	void FUAFMakeDynamicAdditiveAnimOp::SetMeshSpaceAdditive(const bool bInMeshSpaceAdditive)
	{
		bMeshSpaceAdditive = bInMeshSpaceAdditive;
	}
}
