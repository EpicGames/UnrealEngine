// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFNullAnimOp.h"

#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"

namespace UE::UAF
{
	namespace Private
	{
		// A global singleton for all Null AnimOps
		static FUAFNullAnimOp GNullAnimOp;
	}

	FUAFNullAnimOp::FUAFNullAnimOp()
		: FUAFAnimOp(0)
	{
		InitializeAs<FUAFNullAnimOp>();
	}

	FUAFNullAnimOp* FUAFNullAnimOp::Get()
	{
		return &Private::GNullAnimOp;
	}

	void FUAFNullAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

		FPoseValueBundleStack EmptyBundle(AnimOpEvaluationContext.GetNamedSet());
		Evaluator.GetEvaluationStack().Push(FPoseValueBundleCoWRef::MakeFrom(MoveTemp(EmptyBundle)));
	}
}
