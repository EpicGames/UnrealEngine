// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFOffsetRootBoneAnimOp.h"

#include "Animation/AnimRootMotionProvider.h"
#include "Math/SpringMath.h"
#include "BoneIndices.h"
#include "HAL/IConsoleManager.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/Attributes/AttributeBindingIndex.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "VisualLogger/VisualLogger.h"

bool gAnimNodeOffsetRootBoneEnabled = true;
static FAutoConsoleVariableRef CVarUAFOffsetRootBoneEnabled(
	TEXT("a.UAF.OffsetRootBone.Enabled"),
	gAnimNodeOffsetRootBoneEnabled,
	TEXT("True will enable Offset Root Bone for UAF. Equivalent to setting alpha to non-zero.")
);

namespace UE::UAF::OffsetRootBone
{

static bool ShouldExtractRootMotion(const EUAFOffsetRootBoneNodeMode OffsetMode)
{
	switch (OffsetMode)
	{
	case EUAFOffsetRootBoneNodeMode::Accumulate:
	case EUAFOffsetRootBoneNodeMode::Interpolate:
		return true;
	case EUAFOffsetRootBoneNodeMode::Release:
	default:
		return false;
	}
}

static bool ShouldCounterComponentDelta(const EUAFOffsetRootBoneNodeMode OffsetMode)
{
	switch (OffsetMode)
	{
	case EUAFOffsetRootBoneNodeMode::Accumulate:
	case EUAFOffsetRootBoneNodeMode::Interpolate:
		return false;
	case EUAFOffsetRootBoneNodeMode::Release:
	default:
		return true;
	}
}

} // namespace UE::UAF::OffsetRootBone

namespace UE::UAF
{

FUAFOffsetRootBoneAnimOp::FUAFOffsetRootBoneAnimOp() : FUAFAnimOp(1)
{
	InitializeAs<FUAFOffsetRootBoneAnimOp>();
}

void FUAFOffsetRootBoneAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
{
	QUICK_SCOPE_CYCLE_COUNTER(FUAFOffsetRootBoneAnimOp_EvaluateValues);

	if (DeltaTime == 0.f)
	{
		return;
	}

	if (Alpha == 0.f)
	{
		bIsFirstUpdate = true;
		return;
	}

	if (!gAnimNodeOffsetRootBoneEnabled)
	{
		bIsFirstUpdate = true;
		return;
	}

	// Access the value bundle from the evaluation context
	const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();
	FPoseValueBundleCoWRef* ValueBundle = Evaluator.GetEvaluationStack().PeekMutable();
	check(ValueBundle);

	if (ValueBundle->Get().IsEmpty())
	{
		return;
	}
	
	FAttributeNamedSetPtr NamedSet = ValueBundle->Get().GetNamedSet();
	if (!NamedSet.IsValid())
	{
		return;
	}

	// --- Read root bone transform ---
	FAttributeTypedSetPtr BoneTypedSet = NamedSet->FindTypedSet<FBoneTransformAnimationAttribute>();
	if (!BoneTypedSet.IsValid())
	{
		return;
	}

	const FAttributeBindingIndex RootBoneBindingIndex(FSkeletonPoseBoneIndex(0));
	const FAttributeSetIndex RootBoneSetIndex = BoneTypedSet->GetIndex(RootBoneBindingIndex);
	if (!RootBoneSetIndex.IsValid())
	{
		return;
	}

	TBoundValueMap<FBoneTransformAnimationAttribute>* BoneMap = ValueBundle->GetMutable().FindBoneTransforms();
	if (!BoneMap)
	{
		return;
	}

	const FTransform InputBoneTransform = (*BoneMap)[RootBoneSetIndex].Value;

	// --- Read root motion delta ---
	FTransform RootMotionTransformDelta = FTransform::Identity;
	bool bHasRootMotion = false;
	if (FAttributeTypedSetPtr RootMotionTypedSet = NamedSet->FindTypedSet<FTransformAnimationAttribute>())
	{
		const FAttributeSetIndex RootMotionSetIndex = RootMotionTypedSet->FindIndex(UE::Anim::IAnimRootMotionProvider::AttributeName);
		if (RootMotionSetIndex.IsValid())
		{
			FAttributeMappingKey RootMotionMappingKey = FAttributeMappingKey::MakeFromTo<FTransformAnimationAttribute>();
			if (const TBoundValueMap<FTransformAnimationAttribute>* RootMotionMap = ValueBundle->Get().GetBoundValueMaps().Find<FTransformAnimationAttribute>(RootMotionMappingKey))
			{
				RootMotionTransformDelta = (*RootMotionMap)[RootMotionSetIndex].Value;
				bHasRootMotion = true;
			}
		}
	}

	if (!bHasRootMotion)
	{
		return;
	}

	// --- Offset Root Bone calculation ---
	// Ported from FAnimNextOffsetRootBoneTask::Execute

	const FTransform ComponentTransform = MeshComponentTransformWorld;

	if (bIsFirstUpdate)
	{
		SimulatedTranslation = ComponentTransform.GetTranslation();
		SimulatedRotation = ComponentTransform.GetRotation();
	}

	// Note we set last to current on first update
	FTransform LastComponentTransform = bIsFirstUpdate ? ComponentTransform : LastMeshComponentTransformWorld;
	bIsFirstUpdate = false;

	// Teleport detection: if the difference between the previous and current component transform
	// is above the teleport threshold, adjust the simulated position to account for the teleport
	if (FVector::Distance(LastComponentTransform.GetLocation(), ComponentTransform.GetLocation()) > TeleportDistanceThreshold)
	{
		if (bResetOnTeleport)
		{
			SimulatedTranslation = ComponentTransform.GetTranslation();
			SimulatedRotation = ComponentTransform.GetRotation();
		}
		else
		{
			const FTransform OffsetTransform = FTransform(SimulatedRotation, SimulatedTranslation).GetRelativeTransformReverse(LastComponentTransform);
			const FTransform TeleportedTransform = OffsetTransform * ComponentTransform;
			SimulatedTranslation = TeleportedTransform.GetLocation();
			SimulatedRotation = TeleportedTransform.GetRotation();
		}

		LastComponentTransform = ComponentTransform;
	}

	// Cycle last transform
	LastMeshComponentTransformWorld = MeshComponentTransformWorld;

	const EUAFOffsetRootBoneNodeMode CurrentTranslationMode = TranslationMode;
	const EUAFOffsetRootBoneNodeMode CurrentRotationMode = RotationMode;

	const bool bShouldConsumeTranslationOffset = UE::UAF::OffsetRootBone::ShouldExtractRootMotion(CurrentTranslationMode);
	const bool bShouldConsumeRotationOffset = UE::UAF::OffsetRootBone::ShouldExtractRootMotion(CurrentRotationMode);

	RootMotionTransformDelta.NormalizeRotation();

	FTransform ConsumedRootMotionDelta = FTransform::Identity;

	if (bShouldConsumeTranslationOffset)
	{
		ConsumedRootMotionDelta.SetTranslation(RootMotionTransformDelta.GetTranslation());
	}
	if (bShouldConsumeRotationOffset)
	{
		ConsumedRootMotionDelta.SetRotation(RootMotionTransformDelta.GetRotation());
	}

	if (UE::UAF::OffsetRootBone::ShouldCounterComponentDelta(CurrentRotationMode))
	{
		// Accumulate the rotation component delta into the simulated rotation
		const FQuat ComponentRotationDelta = LastComponentTransform.GetRotation().Inverse() * ComponentTransform.GetRotation();
		SimulatedRotation = ComponentRotationDelta * SimulatedRotation;
	}
	if (UE::UAF::OffsetRootBone::ShouldCounterComponentDelta(CurrentTranslationMode))
	{
		// Accumulate the translation component delta into the simulated translation
		const FVector ComponentTranslationDelta = ComponentTransform.GetLocation() - LastComponentTransform.GetLocation();
		SimulatedTranslation += ComponentTranslationDelta;
	}

	FTransform SimulatedTransform(SimulatedRotation, SimulatedTranslation);
	// Apply the root motion delta
	SimulatedTransform = ConsumedRootMotionDelta * SimulatedTransform;

	SimulatedTranslation = SimulatedTransform.GetLocation();
	SimulatedRotation = SimulatedTransform.GetRotation();

	if (bOnGround)
	{
		SimulatedTranslation = FVector::PointPlaneProject(SimulatedTranslation, ComponentTransform.GetLocation(), AnimatedGroundNormal);
	}

	if (TranslationMode == EUAFOffsetRootBoneNodeMode::Release ||
		TranslationMode == EUAFOffsetRootBoneNodeMode::Interpolate)
	{
		FVector TranslationOffset = ComponentTransform.GetLocation() - SimulatedTranslation;

		// Blend out translation offset
		FVector TranslationOffsetDelta = FVector::ZeroVector;
		FMath::ExponentialSmoothingApprox(TranslationOffsetDelta, TranslationOffset, DeltaTime, TranslationSmoothingTime);

		if (bClampToTranslationVelocity)
		{
			const float RootMotionDelta = RootMotionTransformDelta.GetLocation().Size();
			const float MaxDelta = TranslationSpeedRatio * RootMotionDelta;

			const float AdjustmentDelta = TranslationOffsetDelta.Size();
			if (AdjustmentDelta > MaxDelta)
			{
				TranslationOffsetDelta = MaxDelta * TranslationOffsetDelta.GetSafeNormal2D();
			}
		}

		SimulatedTranslation = SimulatedTranslation + TranslationOffsetDelta;
	}

	if (RotationMode == EUAFOffsetRootBoneNodeMode::Release ||
		RotationMode == EUAFOffsetRootBoneNodeMode::Interpolate)
	{
		FQuat RotationOffset = ComponentTransform.GetRotation() * SimulatedRotation.Inverse();
		RotationOffset.EnforceShortestArcWith(FQuat::Identity);
		FQuat RotationOffsetDelta = FQuat::Identity;
		SpringMath::ExponentialSmoothingApproxQuat(RotationOffsetDelta, RotationOffset, DeltaTime, RotationSmoothingTime);

		if (bClampToRotationVelocity)
		{
			float RotationMotionAngleDelta;
			FVector RootMotionRotationAxis;
			RootMotionTransformDelta.GetRotation().ToAxisAndAngle(RootMotionRotationAxis, RotationMotionAngleDelta);

			float MaxRotationAngle = RotationSpeedRatio * RotationMotionAngleDelta;

			FVector DeltaAxis;
			float DeltaAngle;
			RotationOffsetDelta.ToAxisAndAngle(DeltaAxis, DeltaAngle);

			if (DeltaAngle > MaxRotationAngle)
			{
				RotationOffsetDelta = FQuat(DeltaAxis, MaxRotationAngle);
			}
		}

		SimulatedRotation = RotationOffsetDelta * SimulatedRotation;
	}

	if (MaxTranslationError >= 0.0f)
	{
		FVector TranslationOffset = ComponentTransform.GetLocation() - SimulatedTranslation;
		const float TranslationOffsetSizeSquared = TranslationOffset.SizeSquared();
		if (TranslationOffsetSizeSquared > (MaxTranslationError * MaxTranslationError))
		{
			TranslationOffset = TranslationOffset.GetClampedToMaxSize(MaxTranslationError);
			SimulatedTranslation = ComponentTransform.GetLocation() - TranslationOffset;
		}
	}

	const float MaxAngleRadians = FMath::DegreesToRadians(MaxRotationErrorDegrees);
	if (MaxAngleRadians >= 0.0f)
	{
		FQuat RotationOffset = ComponentTransform.GetRotation().Inverse() * SimulatedRotation;
		RotationOffset.EnforceShortestArcWith(FQuat::Identity);

		FVector OffsetAxis;
		float OffsetAngle;
		RotationOffset.ToAxisAndAngle(OffsetAxis, OffsetAngle);

		if (FMath::Abs(OffsetAngle) > MaxAngleRadians)
		{
			RotationOffset = FQuat(OffsetAxis, MaxAngleRadians);
			SimulatedRotation = RotationOffset * ComponentTransform.GetRotation();
			SimulatedRotation.Normalize();
		}
	}

	// Apply the offset adjustments to the simulated transform
	SimulatedTransform.SetLocation(SimulatedTranslation);
	SimulatedTransform.SetRotation(SimulatedRotation);

	// Start with the input pose's bone transform, to preserve any adjustments done before this node in the graph
	FTransform TargetBoneTransform = InputBoneTransform;
	// Accumulate the simulated transform in, and counter current component transform
	TargetBoneTransform.Accumulate(SimulatedTransform * ComponentTransform.Inverse());

	// Offset root bone should not affect scale so take the input
	TargetBoneTransform.SetScale3D(InputBoneTransform.GetScale3D());

	TargetBoneTransform.NormalizeRotation();
	check(SimulatedTranslation.ContainsNaN() == false);

	// Write modified root bone transform back
	(*BoneMap)[RootBoneSetIndex].Value = TargetBoneTransform;

#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
	if (FVisualLogger::IsRecording())
	{
		static const TCHAR* LogName = TEXT("OffsetRootBone");
		const float InnerCircleRadius = 40.0f;
		const uint16 CircleThickness = 2;
		const FVector CircleOffset(0, 0, 1);

		const FTransform TargetBoneInitialTransformWorld = InputBoneTransform * ComponentTransform;
		const FTransform TargetBoneTransformWorld = TargetBoneTransform * ComponentTransform;

		if (MaxTranslationError >= 0.0f)
		{
			const float OuterCircleRadius = MaxTranslationError + InnerCircleRadius;
			UE_VLOG_CIRCLE_THICK(HostObject, TEXT("OffsetRootBone"), Display, ComponentTransform.GetLocation() + CircleOffset, FVector::UpVector, OuterCircleRadius, FColor::Red, CircleThickness, TEXT(""));
		}

		UE_VLOG_CIRCLE_THICK(HostObject, LogName, Display, ComponentTransform.GetLocation() + CircleOffset, FVector::UpVector, InnerCircleRadius, FColor::Blue, CircleThickness, TEXT(""));
		UE_VLOG_ARROW(HostObject, LogName, Display,
			ComponentTransform.GetLocation() + CircleOffset,
			ComponentTransform.GetLocation() + InnerCircleRadius * ComponentTransform.GetRotation().GetRightVector() + CircleOffset,
			FColor::Blue, TEXT(""));

		UE_VLOG_CIRCLE_THICK(HostObject, LogName, Display, TargetBoneTransformWorld.GetLocation() + CircleOffset, FVector::UpVector, InnerCircleRadius, FColor::Green, CircleThickness, TEXT(""));
		UE_VLOG_ARROW(HostObject, LogName, Display,
			TargetBoneTransformWorld.GetLocation() + CircleOffset,
			TargetBoneTransformWorld.GetLocation() + InnerCircleRadius * TargetBoneTransformWorld.GetRotation().GetRightVector() + CircleOffset,
			FColor::Green, TEXT(""));
	}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
}

} // namespace UE::UAF
