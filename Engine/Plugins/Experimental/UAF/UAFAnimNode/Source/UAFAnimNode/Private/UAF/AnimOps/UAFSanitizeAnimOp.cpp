// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFSanitizeAnimOp.h"

#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"

namespace UE::UAF
{
	FUAFSanitizeAnimOp::FUAFSanitizeAnimOp()
		: FUAFAnimOp(1)
	{
		InitializeAs<FUAFSanitizeAnimOp>();
	}

	void FUAFSanitizeAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		FPoseValueBundleCoWRef* InputRef = Evaluator.GetEvaluationStack().PeekMutable();

		if (!InputRef->Get().IsEmpty())
		{
			Transformers::FSanitize::Apply(Evaluator.GetTransformerMap(), InputRef->GetMutable());
		}
	}
}
