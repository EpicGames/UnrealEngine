// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimOps/UAFSteeringAnimOp.h"

#include "Animation/AnimRootMotionProvider.h"
#include "Math/SpringMath.h"
#include "HAL/IConsoleManager.h"
#include "UAF/AnimNodes/IUAFAnimNodeTimeline.h"
#include "UAF/AnimNodes/IUAFRootMotionProvider.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "VisualLogger/VisualLogger.h"

bool gAnimNodeSteeringEnabled = true;
static FAutoConsoleVariableRef CVarUAFSteeringEnabled(
	TEXT("a.UAF.Steering.Enabled"),
	gAnimNodeSteeringEnabled,
	TEXT("True will enable steering for UAF. Equivalent to setting alpha to non-zero.")
);

namespace UE::UAF
{

FUAFSteeringAnimOp::FUAFSteeringAnimOp() : FUAFAnimOp(1)
{
	InitializeAs<FUAFSteeringAnimOp>();
}

void FUAFSteeringAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
{
	if (DeltaTime <= UE_SMALL_NUMBER)
	{
		return;
	}

	if (Alpha <= UE_SMALL_NUMBER)
	{
		return;
	}

	if (gAnimNodeSteeringEnabled == false)
	{
		return;
	}

	// Get the root motion attribute from the evaluator
	FTransform ThisFrameRootMotionTransform = FTransform::Identity;
	

	// Feedback: Would not have been able to discover this API myself, had to read UAFDecompressAnimSequenceAnimOp
	const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

	FPoseValueBundleCoWRef* ValueBundle = Evaluator.GetEvaluationStack().PeekMutable();
	check(ValueBundle != nullptr);

	if (ValueBundle->Get().IsEmpty())
	{
		return;
	}
	
	FAttributeNamedSetPtr NamedSet = ValueBundle->Get().GetNamedSet();
	if (NamedSet.IsValid() == false)
	{
		return;
	}
	
	FAttributeTypedSetPtr AttributeTypedSet = NamedSet->FindTypedSet<FTransformAnimationAttribute>();
	if (AttributeTypedSet.IsValid() == false)
	{
		return;
	}

	// AttributeName: RootMotionDelta
	const FAttributeSetIndex SetIndex = AttributeTypedSet->FindIndex(UE::Anim::IAnimRootMotionProvider::AttributeName);
	if(SetIndex.IsValid() == false)
	{
		return;
	}
	
	FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo<FTransformAnimationAttribute>();
	TBoundValueMap<FTransformAnimationAttribute>* Map = ValueBundle->GetMutable().GetBoundValueMaps().Find<FTransformAnimationAttribute>(MappingKey);
	checkf(Map != nullptr, TEXT("This attribute exists within our named set but isn't present in the output value bundle"));

	ThisFrameRootMotionTransform = (*Map)[SetIndex].Value;

	float CurrentSpeed = ThisFrameRootMotionTransform.GetTranslation().Length() / DeltaTime;
	if (CurrentSpeed < DisableSteeringBelowSpeed)
	{
		return;
	}

	FQuat RootBoneRotation = RootBoneTransform.GetRotation();
#if ENABLE_ANIM_DEBUG
	UE_VLOG_ARROW(HostObject, "Steering", Display,
		RootBoneTransform.GetLocation(),
		RootBoneTransform.GetLocation() + RootBoneRotation.GetRightVector() * 90,
		FColor::Green, TEXT(""));

	UE_VLOG_ARROW(HostObject, "Steering", Display,
		RootBoneTransform.GetLocation(),
		RootBoneTransform.GetLocation() + TargetOrientation.GetRightVector() * 100,
		FColor::Blue, TEXT(""));
#endif // ENABLE_ANIM_DEBUG

	FQuat DeltaToTargetOrientation = RootBoneRotation.Inverse() * TargetOrientation;

	if (Timeline != nullptr && RootMotionProvider != nullptr && AnimatedTargetTime > 0.0f)
	{
		// Root motion prediction
		float CurrentTime = Timeline->GetCurrentTime();

		FTransform PredictedRootMotionTransform = RootMotionProvider->ExtractRootMotion(CurrentTime, AnimatedTargetTime, Timeline->IsLooping());
		FQuat PredictedRootMotionQuat = PredictedRootMotionTransform.GetRotation();
		FRotator PredictedRootMotionRot = FRotator(PredictedRootMotionQuat);
		float PredictedRootMotionYaw = PredictedRootMotionRot.Yaw;

		if (fabs(PredictedRootMotionYaw) > RootMotionAngleThresholdDegrees)
		{
#if ENABLE_ANIM_DEBUG
			UE_VLOG_ARROW(HostObject, "Steering", Display,
				RootBoneTransform.GetLocation(),
				RootBoneTransform.GetLocation() + (PredictedRootMotionQuat * RootBoneRotation).GetRightVector() * 100,
				FColor::Orange, TEXT(""));
#endif // ENABLE_ANIM_DEBUG 

			float YawToTargetOrientation = FRotator(DeltaToTargetOrientation).Yaw;

			// pick the rotation direction that is the shortest path from the endpoint of the current animated rotation
			if (PredictedRootMotionYaw - YawToTargetOrientation > 180)
			{
				YawToTargetOrientation += 360;
			}
			else if (YawToTargetOrientation - PredictedRootMotionYaw > 180)
			{
				YawToTargetOrientation -= 360;
			}

			float Ratio = YawToTargetOrientation / PredictedRootMotionYaw;
			Ratio = FMath::Clamp(Ratio, MinScaleRatio, MaxScaleRatio);

			// Account for alpha
			Ratio = FMath::Lerp(1.0f, Ratio, Alpha);

			FRotator ThisFrameRootMotionRotator(ThisFrameRootMotionTransform.GetRotation());
			ThisFrameRootMotionRotator.Yaw *= Ratio;
			ThisFrameRootMotionTransform.SetRotation(FQuat(ThisFrameRootMotionRotator));

			// Account for future scaling in linear error correction
			PredictedRootMotionRot.Yaw *= Ratio;
			PredictedRootMotionQuat = PredictedRootMotionRot.Quaternion();

			DeltaToTargetOrientation = PredictedRootMotionQuat.Inverse() * RootBoneRotation.Inverse() * TargetOrientation;
		}
	}

	if (CurrentSpeed > DisableAdditiveBelowSpeed)
	{
		// Apply linear correction
		FQuat LinearCorrection = FQuat::Identity;
		SpringMath::CriticalSpringDamperQuat(LinearCorrection, AngularVelocity, DeltaToTargetOrientation, SpringMath::HalfLifeToSmoothingTime(ProceduralTargetTime), DeltaTime);

#if ENABLE_ANIM_DEBUG
		UE_VLOG_ARROW(HostObject, "Steering", Display,
			RootBoneTransform.GetLocation(),
			RootBoneTransform.GetLocation() + (RootBoneTransform.GetRotation() * LinearCorrection).GetRightVector() * 120,
			FColor::Magenta, TEXT(""));
#endif

		FQuat ThisFrameRotation = ThisFrameRootMotionTransform.GetRotation() * LinearCorrection;

#if ENABLE_ANIM_DEBUG
		UE_VLOG_ARROW(HostObject, "Steering", Display,
			RootBoneTransform.GetLocation(),
			RootBoneTransform.GetLocation() + (RootBoneTransform.GetRotation() * ThisFrameRotation).GetRightVector() * 140,
			FColor::Red, TEXT(""));
#endif

		ThisFrameRootMotionTransform.SetRotation(FQuat::Slerp(ThisFrameRootMotionTransform.GetRotation(), ThisFrameRotation, Alpha));
	}

	// Set back the updated root motion transform
	(*Map)[SetIndex].Value = ThisFrameRootMotionTransform;
}
}
