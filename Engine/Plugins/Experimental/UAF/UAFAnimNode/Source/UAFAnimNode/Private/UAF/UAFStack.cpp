// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/UAFStack.h"

namespace UE::UAF
{
	const FUAFStack::FEntryHeader* FUAFStack::PeekImpl(uint32 OffsetFromTop) const
	{
		const FEntryHeader* EntryHeader = Top;

		uint32 Count = 0;
		while (EntryHeader != nullptr && Count < OffsetFromTop)
		{
			EntryHeader = EntryHeader->PrevTop;
			Count++;
		}

		if (Count != OffsetFromTop)
		{
			// Offset is too far, not enough entries
			checkf(false, TEXT("Attempting to peek at an element that exceeds the stack size."));
			return nullptr;
		}

		check(EntryHeader != nullptr);
		return EntryHeader;
	}
}
