// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNodeFactory.h"

#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "Templates/SubclassOf.h"

namespace UE::UAF
{
TMap<FTopLevelAssetPath, FUAFAnimNodeFactory::FInitializerInfo> FUAFAnimNodeFactory::AssetInitializers;

const FUAFAnimNodeFactory::FInitializerInfo* FUAFAnimNodeFactory::FindAssetInitializer(const UClass* Class)
{
	check(Class != nullptr);
	FInitializerInfo* InitializerInfo;
	const UClass* SearchClass = Class;
	do
	{
		InitializerInfo = AssetInitializers.Find(SearchClass->GetClassPathName());
		SearchClass = SearchClass->GetSuperClass();
	}
	while (InitializerInfo == nullptr && SearchClass != nullptr);
	return InitializerInfo;
}

FTopLevelAssetPath FUAFAnimNodeFactory::RegisterAsset(UClass* ObjectClass, FAssetInitializer&& Initializer)
{
	const FTopLevelAssetPath ClassPathName = ObjectClass->GetClassPathName();
	
	// todo: we may want to support this for licensee overrides
	checkf(AssetInitializers.Contains(ClassPathName) == false, TEXT("Attempting to register type %s in UAF::FUAFAnimNodeFactory twice"), *ObjectClass->GetFullName());

	AssetInitializers.Add(ClassPathName, FInitializerInfo(Initializer));
	return ClassPathName;
}

FUAFAnimNodePtr FUAFAnimNodeFactory::CreateUAFAnimNodeFromObject(UObject* Object, FUAFAnimGraphUpdateContext& Context)
{
	check(Object != nullptr);
	
	const FInitializerInfo* InitializerInfo = FindAssetInitializer(Object->GetClass());
	if (ensureMsgf(InitializerInfo != nullptr, TEXT("Unsupported object type (%s) passed to UAF::FAnimNodeFactory"), *Object->GetClass()->GetFullName()))
	{
		return InitializerInfo->Initializer(Object, Context);
	}

	return nullptr;
}

void FUAFAnimNodeFactory::UnregisterAsset(const FTopLevelAssetPath& InObjectClassPath)
{
	check(InObjectClassPath.IsValid());

	int Removed = AssetInitializers.Remove(InObjectClassPath);
	ensure(Removed > 0);
}

bool FUAFAnimNodeFactory::IsAssetRegistered(const UClass* AssetClass)
{
	return FindAssetInitializer(AssetClass) != nullptr;
}

}


