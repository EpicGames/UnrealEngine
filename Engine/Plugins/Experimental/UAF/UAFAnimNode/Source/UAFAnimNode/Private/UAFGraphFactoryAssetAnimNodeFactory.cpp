// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFGraphFactoryAssetAnimNodeFactory.h"

#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "Templates/SubclassOf.h"

namespace UE::UAF
{
TMap<FTopLevelAssetPath, FUAFGraphFactoryAssetAnimNodeFactory::FInitializerInfo> FUAFGraphFactoryAssetAnimNodeFactory::AssetInitializers;

const FUAFGraphFactoryAssetAnimNodeFactory::FInitializerInfo* FUAFGraphFactoryAssetAnimNodeFactory::FindAssetInitializer(const UStruct* Struct)
{
	check(Struct != nullptr);
	FInitializerInfo* InitializerInfo;
	const UStruct* SearchStruct = Struct;
	do
	{
		InitializerInfo = AssetInitializers.Find(SearchStruct->GetStructPathName());
		SearchStruct = SearchStruct->GetSuperStruct();
	}
	while (InitializerInfo == nullptr && SearchStruct != nullptr);
	return InitializerInfo;
}

FTopLevelAssetPath FUAFGraphFactoryAssetAnimNodeFactory::RegisterStruct(TSubScriptStructOf<FUAFGraphFactoryAsset> Struct, FUAFGraphFactoryAssetAnimNodeFactory::FAssetInitializer&& Initializer)
{
	const FTopLevelAssetPath StructPathName = Struct->GetStructPathName();
	
	// todo: we may want to support this for licensee overrides
	checkf(AssetInitializers.Contains(StructPathName) == false, TEXT("Attempting to register type %s in UAF::FUAFGraphFactoryAssetAnimNodeFactory twice"), *Struct->GetFullName());

	AssetInitializers.Add(StructPathName, FInitializerInfo(Initializer));
	return StructPathName;
}

FUAFAnimNodePtr FUAFGraphFactoryAssetAnimNodeFactory::CreateUAFAnimNodeFromObject(TConstStructView<FUAFGraphFactoryAsset> Value, FUAFAnimGraphUpdateContext& Context)
{
	check(Value.IsValid());
	
	const FInitializerInfo* InitializerInfo = FindAssetInitializer(Value.GetScriptStruct());
	if (ensureMsgf(InitializerInfo != nullptr, TEXT("Unsupported struct type (%s) passed to UAF::FAnimNodeFactory"), *Value.GetScriptStruct()->GetFullName()))
	{
		return InitializerInfo->Initializer(Value, Context);
	}

	return nullptr;
}

void FUAFGraphFactoryAssetAnimNodeFactory::UnregisterStruct(const FTopLevelAssetPath& InObjectClassPath)
{
	check(InObjectClassPath.IsValid());

	int Removed = AssetInitializers.Remove(InObjectClassPath);
	ensure(Removed > 0);
}

bool FUAFGraphFactoryAssetAnimNodeFactory::IsStructRegistered(TSubScriptStructOf<FUAFGraphFactoryAsset> InStructType)
{
	return FindAssetInitializer(InStructType) != nullptr;
}

}


