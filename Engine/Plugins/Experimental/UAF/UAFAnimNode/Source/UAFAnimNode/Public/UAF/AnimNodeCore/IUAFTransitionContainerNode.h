// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeInterfaceId.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

namespace UE::UAF
{
	struct IUAFTransitionNode;

	/**
	 * IUAFTransitionContainerNode
	 *
	 * Interface implemented by Nodes which use transitions, in order to be notified when the transition completes
	 */
	struct IUAFTransitionContainerNode
	{
		static constexpr FUAFAnimNodeInterfaceId InterfaceId = FUAFAnimNodeInterfaceId::MakeFromString(TEXT("IUAFTransitionContainerNode"));

		virtual ~IUAFTransitionContainerNode() = default;

		// Called by transitions when they complete
		virtual void NotifyTransitionComplete(const IUAFTransitionNode& TransitionNode) = 0;
	};
	
}
