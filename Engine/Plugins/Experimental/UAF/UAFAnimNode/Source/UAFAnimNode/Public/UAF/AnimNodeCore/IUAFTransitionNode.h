// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeInterfaceId.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * IUAFTransitionNode
	 *
	 * Interface implemented by blend transitions
	 */
	struct IUAFTransitionNode
	{
		static constexpr FUAFAnimNodeInterfaceId InterfaceId = FUAFAnimNodeInterfaceId::MakeFromString(TEXT("IUAFTransitionNode"));

		virtual ~IUAFTransitionNode() = default;

		// Returns the source node we are transitioning away from
		[[nodiscard]] virtual const FUAFAnimNodePtr& GetSource() const = 0;

		// Returns the target node we are transitioning towards to
		[[nodiscard]] virtual const FUAFAnimNodePtr& GetTarget() const = 0;

		// Un-parents the source node and returns it
		// Can only be called when the transition has completed as this is used to steal ownership
		[[nodiscard]] virtual FUAFAnimNodePtr ReleaseSource() = 0;

		// Un-parents the target node and returns it
		// Can only be called when the transition has completed as this is used to steal ownership
		[[nodiscard]] virtual FUAFAnimNodePtr ReleaseTarget() = 0;

		// Returns whether or not the transition has completed
		[[nodiscard]] virtual bool IsComplete() const = 0;
		
		// Call Prune on a completed transition to get the unparented Target node
		// Prune will also check if current target is another completed IUAFTransition,
		// and will prune all the way down to a target node which is either not a transition, or not complete
		[[nodiscard]] FUAFAnimNodePtr Prune()
		{
			ensure(IsComplete());
			
			FUAFAnimNodePtr Target = ReleaseTarget();

			while(Target)
			{
				// If we contain any finished transtions prune all of them
				if (IUAFTransitionNode* TransitionInstance = Target->GetInterface<IUAFTransitionNode>())
				{
					if (TransitionInstance->IsComplete())
					{
						Target = TransitionInstance->ReleaseTarget();
						continue;
					}
				}
				break;
			} 

			return Target;
		}
	};
}

#undef UE_API
