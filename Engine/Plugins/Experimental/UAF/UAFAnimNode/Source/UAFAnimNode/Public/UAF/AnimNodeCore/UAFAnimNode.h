// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationProgram.h"
#include "Templates/RefCounting.h"
#include "UAF/AnimNodeCore/UAFAnimNodeInterfaceId.h"
#include "UAF/AnimNodeCore/UAFAnimNodeReferenceCollector.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"
#include "UAF/AnimNodeCore/UAFAnimNodeTrace.h"

#define UE_API UAFANIMNODE_API

struct FAnimNextEvaluationTask;
class FReferenceCollector;

namespace UE::UAF
{
	// Smart pointer to an instance of FUAFAnimNode
	using FUAFAnimNodePtr = TRefCountPtr<class FUAFAnimNode>;

	/**
	 * FUAFAnimNode
	 *
	 * An anim node instance
	 */
	class FUAFAnimNode
	{
	public:
		UE_NONCOPYABLE(FUAFAnimNode);	// We disallow copy/move semantics

		// We prohibit creating an anim node without an update context
		FUAFAnimNode() = delete;

		// Resets this node to its initial state
		// This must be called when a node is re-used
		// Cannot be called if the node is attached to a parent
		UE_API void Reset();

		// Returns the child at the specified index
		[[nodiscard]] const FUAFAnimNodePtr& GetChildAt(int32 ChildIndex) const;

		// Returns the first child
		[[nodiscard]] const FUAFAnimNodePtr& GetFirstChild() const;

		// Returns the node's parent
		[[nodiscard]] FUAFAnimNode* GetParent() const;

		// Returns the number of children held by this node
		[[nodiscard]] int32 GetNumChildren() const;

		// Returns whether or not this node holds children
		[[nodiscard]] bool HasChildren() const;

		// Returns the list of children held by this node
		[[nodiscard]] const TArray<FUAFAnimNodePtr, TInlineAllocator<2>>& GetChildren() const;

		// Returns the current total weight factoring in all inherited blend weights
		[[nodiscard]] float GetTotalWeight() const;

		// Sets the total weight
		void SetTotalWeight(float Weight);

		// Returns whether or not we are blending out
		[[nodiscard]] bool IsBlendingOut() const;

		// Sets whether or not we are blending out
		void SetIsBlendingOut(bool bIsBlendingOut);

		// Returns whether or not we are newly relevant
		// A node is newly relevant the first time it updates
		// If you wish to re-use a node after it has fully blended out, you must first reset it
		[[nodiscard]] bool IsNewlyRelevant() const;

		// Hints that this node is newly relevant
		// This should be used when nodes are manually cached and re-used to hint when they become relevant again
		// Note that if the node is already relevant, it cannot be made irrelevant until it updates
		void HintNewlyRelevant(bool bIsNewlyRelevant);

		// Returns the AnimOp to perform before our children evaluate theirs
		[[nodiscard]] FUAFAnimOp* GetPreAnimOp();
		[[nodiscard]] const FUAFAnimOp* GetPreAnimOp() const;

		// Returns the AnimOp to perform after our children evaluate theirs
		[[nodiscard]] FUAFAnimOp* GetPostAnimOp();
		[[nodiscard]] const FUAFAnimOp* GetPostAnimOp() const;

		// Returns a pointer to the specified interface or nullptr if the interface is not supported by the current node data
		template<typename InterfaceType> [[nodiscard]] InterfaceType* GetInterface();
		template<typename InterfaceType> [[nodiscard]] const InterfaceType* GetInterface() const;
		[[nodiscard]] virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) = 0;

		// Called when garbage collection requests hard/strong object references
		// @see UObject::AddReferencedObjects
		static void AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector);
		
#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const;
		UE_API virtual UStruct* GetDebugStruct() const;
		
		uint64 GetDebugInstanceId() const
		{
			return DebugInstanceId;
		}

		void SetDebugInstanceId(uint64 Id)
		{
			DebugInstanceId = Id;
		}
#endif

#if DO_CHECK
		// Tests the debug update counters to determine if Pre/PostUpdate has been called or not yet
		[[nodiscard]] bool HasPreUpdated() const;
		[[nodiscard]] bool HasPostUpdated() const;
#endif

	protected:
		// Only derived types can construct an anim node
		explicit FUAFAnimNode(FUAFAnimGraphUpdateContext& UpdateContext);

		// Only a friend or derived type can call the destructor to ensure FUAFAnimNodePtr is used for lifetime management
		UE_API virtual ~FUAFAnimNode();

		// Initialization function should be called within the constructor of derived types
		template<class AnimNodeType>
		void InitializeAs(FUAFAnimGraphUpdateContext& UpdateContext);

		// Main update entry point for node instances
		// This function is responsible for updating topology (e.g. managing children) and advancing time/phase
		// Called during the update pass before children have been queued and traversed
		// When choosing whether to use PreUpdate or PostUpdate, favor the former when possible
		// This is 'protected' to ensure that no one attempts to manually update untyped nodes.
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext);

		// Main update entry point for node instances
		// This function is responsible for updating topology (e.g. managing children) and advancing time/phase
		// Called during the update pass after children have been queued and traversed
		// Note that it is not possible to change the PreAnimOp while in PostUpdate as it has already been queued
		// When choosing whether to use PreUpdate or PostUpdate, favor the former when possible
		// This is 'protected' to ensure that no one attempts to manually update untyped nodes.
		UE_API virtual void PostUpdate(FUAFAnimGraphUpdateContext& GraphContext);

		// Adds a new child to this node and returns its index
		UE_API int32 AddChild(FUAFAnimNodePtr&& Child);
		UE_API int32 AddChild(const FUAFAnimNodePtr& Child);

		// Removes the specified child
		// Returns true on success if the child was found, false otherwise
		UE_API bool RemoveChild(const FUAFAnimNodePtr& Child);

		// Removes the child at the specified index
		UE_API void RemoveChildAt(int32 ChildIndex);

		// Sets the child at the specified index
		UE_API void SetChildAt(int32 ChildIndex, const FUAFAnimNodePtr& Child);
		
		// Sets the AnimOp to perform before our children evaluate theirs
		// When choosing whether to use PreAnimOp or PostAnimOp, favor the latter when possible
		void SetPreAnimOp(FUAFAnimOp* AnimOp);

		// Sets the AnimOp to perform after our children evaluate theirs
		// When choosing whether to use PreAnimOp or PostAnimOp, favor the latter when possible
		// Must be used by leaf nodes to produce an output
		void SetPostAnimOp(FUAFAnimOp* AnimOp);

#if UAF_TRACE_ENABLED
		uint64 DebugInstanceId = 0;
#endif
	private:
		// Increments the reference count
		// TRefCountPtr API
		void AddRef() const;

		// Decreases the reference count and queues this node for destruction when the count reaches zero
		// TRefCountPtr API
		void Release() const;

		// Queues this node for destruction within the current update context
		// Don't inline to avoid code bloat
		UE_API FORCENOINLINE void QueueForDestruction();

		// Clears this node's parent (should only be called by SetChild/AddChild/RemoveChild functions)
		void ClearParent();
		
		// Sets this node's parent (should only be called by SetChild/AddChild functions)
		void SetParent(FUAFAnimNode* Parent);

		// A pointer to this node's parent
		// This pointer is valid if the node is attached to a parent - do not modify directly, always call SetParent/ClearParent
		FUAFAnimNode* Parent = nullptr;

		// An array of children owned by this node
		// Note that children can be nullptr
		TArray<FUAFAnimNodePtr, TInlineAllocator<2>> Children;

		// The AnimOps to perform before and after our children evaluate theirs
		FUAFAnimOp* PreAnimOp = nullptr;
		FUAFAnimOp* PostAnimOp = nullptr;

		// The current total weight factoring in all inherited blend weights
		// Nodes with a single child will automatically propagate this value
		float TotalWeight = 0.0f;

		// Whether or not we are blending out
		// Nodes with a single child will automatically propagate this value
		bool bIsBlendingOut : 1 = false;

		// Whether or not we are newly relevant
		// Nodes with a single child will automatically propagate this value
		bool bIsNewlyRelevant : 1 = true;

		// Whether or not this node implements AddReferencedObjects(..)
		bool bHasAddReferencedObjects : 1 = false;

#if DO_CHECK
		// Whether or not InitializeAs<T>(..) has been called
		bool bIsInitialized : 1 = false;

		// Whether or not the last reference was released and we have been queued for destruction
		mutable bool bIsPendingDestroy : 1 = false;

		// Debug counters used to enforce invariants
		uint32 PreUpdateCounter = 0;
		uint32 PostUpdateCounter = 0;
#endif

		// The number of references currently held on this node
		// We only use 8-bits because we don't expect shared ownership semantics over a high number of nodes
		// We are always created with one reference
		mutable uint8 ReferenceCount = 1;

		friend UE_API TArray<FUAFAnimOp*> UpdateGraph(FUAFAnimGraphUpdateContext& UpdateContext, FUAFAnimNode& RootNode);
		friend FUAFAnimGraphUpdateContext;
		friend FUAFAnimNodePtr;
		
		template <class AnimNodeType>
		friend class TUAFInlineAnimNode;
	};

	// Constructs a new anim node of the specified type with the provided arguments and returns a smart pointer to it
	template<class AnimNodeType, typename... ArgTypes>
	FUAFAnimNodePtr MakeAnimNode(FUAFAnimGraphUpdateContext& UpdateContext, ArgTypes&&... Args);

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline FUAFAnimNode::FUAFAnimNode(FUAFAnimGraphUpdateContext& UpdateContext)
	{
	}

	template<class AnimNodeType>
	inline void FUAFAnimNode::InitializeAs(FUAFAnimGraphUpdateContext& UpdateContext)
	{
		// Cannot use std::same_v with static functions since their signature is the same
		bHasAddReferencedObjects = std::conditional_t<&AnimNodeType::AddReferencedObjects != &FUAFAnimNode::AddReferencedObjects, std::true_type, std::false_type>::value;

#if DO_CHECK
		bIsInitialized = true;
#endif

		if (bHasAddReferencedObjects)
		{
			UpdateContext.GetGCReferences().RegisterWithGC(this, &AnimNodeType::AddReferencedObjects);
		}
	}

	inline void FUAFAnimNode::AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector)
	{
	}

#if DO_CHECK
	inline bool FUAFAnimNode::HasPreUpdated() const
	{
		// The counter is updated to match after PreUpdate is called
		return PreUpdateCounter == FUAFAnimGraphUpdateContext::GetCurrentFromTLS()->GetUpdateCounter();
	}

	inline bool FUAFAnimNode::HasPostUpdated() const
	{
		// The counter is updated to match after PostUpdate is called
		return PostUpdateCounter == FUAFAnimGraphUpdateContext::GetCurrentFromTLS()->GetUpdateCounter();
	}
#endif

	inline const FUAFAnimNodePtr& FUAFAnimNode::GetChildAt(int32 ChildIndex) const
	{
		return Children[ChildIndex];
	}

	inline const FUAFAnimNodePtr& FUAFAnimNode::GetFirstChild() const
	{
		return Children[0];
	}
	
	inline FUAFAnimNode* FUAFAnimNode::GetParent() const
	{
		return Parent;
	}

	inline int32 FUAFAnimNode::GetNumChildren() const
	{
		return Children.Num();
	}

	inline bool FUAFAnimNode::HasChildren() const
	{
		return !Children.IsEmpty();
	}

	inline const TArray<FUAFAnimNodePtr, TInlineAllocator<2>>& FUAFAnimNode::GetChildren() const
	{
		return Children;
	}

	inline float FUAFAnimNode::GetTotalWeight() const
	{
		return TotalWeight;
	}

	inline void FUAFAnimNode::SetTotalWeight(float Weight)
	{
#if DO_CHECK
		checkf(!HasPreUpdated(), TEXT("Attempted to set the total weight after PreUpdate has been called. The weight has already propagated down towards its children."));
#endif

		TotalWeight = FMath::Clamp(Weight, 0.0f, 1.0f);
	}

	inline bool FUAFAnimNode::IsBlendingOut() const
	{
		return bIsBlendingOut;
	}

	inline void FUAFAnimNode::SetIsBlendingOut(bool bInIsBlendingOut)
	{
#if DO_CHECK
		checkf(!HasPreUpdated(), TEXT("Attempted to set the blending out flag after PreUpdate has been called. The blending out flag has already propagated down towards its children."));
#endif

		bIsBlendingOut = bInIsBlendingOut;
	}

	inline bool FUAFAnimNode::IsNewlyRelevant() const
	{
		return bIsNewlyRelevant;
	}

	inline void FUAFAnimNode::HintNewlyRelevant(bool bInIsNewlyRelevant)
	{
		bIsNewlyRelevant |= bInIsNewlyRelevant;
	}

	inline FUAFAnimOp* FUAFAnimNode::GetPreAnimOp()
	{
		return PreAnimOp;
	}

	inline const FUAFAnimOp* FUAFAnimNode::GetPreAnimOp() const
	{
		return PreAnimOp;
	}

	inline void FUAFAnimNode::SetPreAnimOp(FUAFAnimOp* AnimOp)
	{
#if DO_CHECK
		checkf(PreAnimOp == nullptr || !HasPreUpdated(), TEXT("Attempted to set the pre-AnimOp after PreUpdate has been called. The previous AnimOp has already been queued."));
#endif

		PreAnimOp = AnimOp;
	}

	inline FUAFAnimOp* FUAFAnimNode::GetPostAnimOp()
	{
		return PostAnimOp;
	}

	inline const FUAFAnimOp* FUAFAnimNode::GetPostAnimOp() const
	{
		return PostAnimOp;
	}

	inline void FUAFAnimNode::SetPostAnimOp(FUAFAnimOp* AnimOp)
	{
#if DO_CHECK
		checkf(PostAnimOp == nullptr || !HasPostUpdated(), TEXT("Attempted to set the post-AnimOp after PostUpdate has been called. The previous AnimOp has already been queued."));
#endif

		PostAnimOp = AnimOp;
	}

	template<typename InterfaceType>
	inline InterfaceType* FUAFAnimNode::GetInterface()
	{
		return static_cast<InterfaceType*>(GetInterface(InterfaceType::InterfaceId));
	}

	template<typename InterfaceType>
	inline const InterfaceType* FUAFAnimNode::GetInterface() const
	{
		return static_cast<const InterfaceType*>(const_cast<FUAFAnimNode*>(this)->GetInterface(InterfaceType::InterfaceId));
	}

	inline void FUAFAnimNode::AddRef() const
	{
		checkf(ReferenceCount != MAX_uint8, TEXT("Attempting to increment the reference count past 255, this will overflow and lead to failure."));
		ReferenceCount++;
	}

	inline void FUAFAnimNode::Release() const
	{
		if (ReferenceCount-- == 1)
		{
			// The last reference was removed, handle destruction
			const_cast<FUAFAnimNode*>(this)->QueueForDestruction();
		}
	}

	inline void FUAFAnimNode::SetParent(FUAFAnimNode* InParent)
	{
		check(InParent);
		Parent = InParent;
	}

	inline void FUAFAnimNode::ClearParent()
	{
		Parent = nullptr;
	}

	template<class AnimNodeType, typename... ArgTypes>
	inline FUAFAnimNodePtr MakeAnimNode(FUAFAnimGraphUpdateContext& UpdateContext, ArgTypes&&... Args)
	{
		// No need to increment the reference, it initializes to 1 already for us
		constexpr bool bAddRef = false;
		return FUAFAnimNodePtr(new AnimNodeType(UpdateContext, Forward<ArgTypes>(Args)...), bAddRef);
	}
}

#undef UE_API
