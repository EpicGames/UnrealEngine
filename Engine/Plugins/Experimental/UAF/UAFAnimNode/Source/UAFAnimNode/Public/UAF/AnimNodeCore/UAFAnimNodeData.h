// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataInterfaceId.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

#include "UAFAnimNodeData.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFAnimNodeData
	 *
	 * Base struct for anim node shared data.
	 * Node data must live at least as long as node instances that use it.
	 */
	USTRUCT()
	struct FUAFAnimNodeData
	{
		GENERATED_BODY()
	
		virtual ~FUAFAnimNodeData() = default;

		// Creates a new node instance based on the current node data
		[[nodiscard]] UE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const;

		// Returns a pointer to the specified interface or nullptr if the interface is not supported by the current node data
		template<typename InterfaceType> [[nodiscard]] InterfaceType* GetInterface();
		template<typename InterfaceType> [[nodiscard]] const InterfaceType* GetInterface() const;

		[[nodiscard]] virtual void* GetInterface(FUAFAnimNodeDataInterfaceId Id);
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	template <typename InterfaceType>
	inline InterfaceType* FUAFAnimNodeData::GetInterface()
	{
		return static_cast<InterfaceType*>(GetInterface(InterfaceType::InterfaceId));
	}

	template <typename InterfaceType>
	inline const InterfaceType* FUAFAnimNodeData::GetInterface() const
	{
		return static_cast<const InterfaceType*>(const_cast<FUAFAnimNodeData*>(this)->GetInterface(InterfaceType::InterfaceId));
	}

	inline void* FUAFAnimNodeData::GetInterface(FUAFAnimNodeDataInterfaceId Id)
	{
		return nullptr;
	}
}

#undef UE_API
