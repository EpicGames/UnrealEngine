// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"

#include "UAFAnimNodeDataEx.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF::AnimNodeEditor
{
	class FUAFAnimNodeDataExDetails;
}

namespace UE::UAF
{
	/**
	 * FUAFAnimNodeDataEx
	 *
	 * Encapsulates a base anim node and a list of modifiers to apply to it.
	 * This struct has custom details panel tooling to generate an anim node from these.
	 */
	USTRUCT(BlueprintType)
	struct FUAFAnimNodeDataEx
	{
		GENERATED_BODY()

		// Creates an anim node from a base and a list of modifiers
		// The base node is wrapped by each modifier (index 0 is the first modifier to wrap the base, index 1 wraps the first modifier, etc)
		static FUAFAnimNodeDataEx Make(const TInstancedStruct<FUAFAnimNodeData>& Base, const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& Modifiers);

		// Builds an anim node from a base and a list of modifiers
		// The base node is wrapped by each modifier (index 0 is the first modifier to wrap the base, index 1 wraps the first modifier, etc)
		UE_API void Build(const TInstancedStruct<FUAFAnimNodeData>& Base, const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& Modifiers);

		// Returns whether or not this anim node is valid
		bool IsValid() const;

		// Returns the generated anim node
		const TInstancedStruct<FUAFAnimNodeData>& Get() const;

#if WITH_EDITORONLY_DATA
		// Sets the base anim node
		// The base will be wrapped by the list of modifiers
		void SetBase(const TInstancedStruct<FUAFAnimNodeData>& Base);

		// Returns the base anim node
		TInstancedStruct<FUAFAnimNodeData>& GetBase();
		const TInstancedStruct<FUAFAnimNodeData>& GetBase() const;

		// Sets the list of modifiers
		// The base node is wrapped by each modifier (index 0 is the first modifier to wrap the base, index 1 wraps the first modifier, etc)
		void SetModifiers(const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& Modifiers);

		// Returns the list of modifiers
		TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& GetModifiers();
		const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& GetModifiers() const;
	
		// StructOps API
		UE_API void PostSerialize(const FArchive& Ar);
		UE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
#endif

	private:
		// Baked anim node generated from the base and modifiers
		UPROPERTY(VisibleAnywhere, Category = "Data", meta = (ExcludeBaseStruct, EditCondition = "bShowBakedNode", EditConditionHides, HideEditConditionToggle))
		TInstancedStruct<FUAFAnimNodeData> BakedNode;

#if WITH_EDITORONLY_DATA
		// Rebuilds the anim node from the editor data
		UE_API void Refresh();

		// Base anim node that we'll wrap
		UPROPERTY(EditAnywhere, Category = "Data", meta = (ExcludeBaseStruct))
		TInstancedStruct<FUAFAnimNodeData> BaseNode;

		// List of modifier anim nodes to apply
		// Modifiers are applied in the order they are added (first wraps the base)
		UPROPERTY(EditAnywhere, Category = "Data", meta = (ExcludeBaseStruct))
		TArray<TInstancedStruct<FUAFModifierAnimNodeData>> ModifierNodes;

		// This flag is used to hide the baked node, can be enabled with a cvar
		UPROPERTY()
		bool bShowBakedNode = false;

		friend AnimNodeEditor::FUAFAnimNodeDataExDetails;
#endif
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline FUAFAnimNodeDataEx FUAFAnimNodeDataEx::Make(const TInstancedStruct<FUAFAnimNodeData>& Base, const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& Modifiers)
	{
		FUAFAnimNodeDataEx Result;
		Result.Build(Base, Modifiers);
		return Result;
	}

	inline bool FUAFAnimNodeDataEx::IsValid() const
	{
		return BakedNode.IsValid();
	}

	inline const TInstancedStruct<FUAFAnimNodeData>& FUAFAnimNodeDataEx::Get() const
	{
		return BakedNode;
	}

#if WITH_EDITORONLY_DATA
	inline void FUAFAnimNodeDataEx::SetBase(const TInstancedStruct<FUAFAnimNodeData>& Base)
	{
		BaseNode = Base;
		Refresh();
	}

	inline TInstancedStruct<FUAFAnimNodeData>& FUAFAnimNodeDataEx::GetBase()
	{
		return BaseNode;
	}

	inline const TInstancedStruct<FUAFAnimNodeData>& FUAFAnimNodeDataEx::GetBase() const
	{
		return BaseNode;
	}

	inline void FUAFAnimNodeDataEx::SetModifiers(const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& Modifiers)
	{
		ModifierNodes = Modifiers;
		Refresh();
	}

	inline TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& FUAFAnimNodeDataEx::GetModifiers()
	{
		return ModifierNodes;
	}

	inline const TArray<TInstancedStruct<FUAFModifierAnimNodeData>>& FUAFAnimNodeDataEx::GetModifiers() const
	{
		return ModifierNodes;
	}
#endif
}

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<UE::UAF::FUAFAnimNodeDataEx> : TStructOpsTypeTraitsBase2<UE::UAF::FUAFAnimNodeDataEx>
{
	enum
	{
		WithPostSerialize = true,
		WithImportTextItem = true,
	};
};
#endif

#undef UE_API
