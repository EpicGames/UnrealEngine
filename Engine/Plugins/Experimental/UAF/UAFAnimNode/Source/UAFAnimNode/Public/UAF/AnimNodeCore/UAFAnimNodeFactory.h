// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/SubScriptStructOf.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

namespace UE::UAF
{
	
/*
 * The FUAFAnimNodeFactory is used to produce FUAFAnimNode instances from UObjects. This is used in the runtime to produce AnimNodes for animation assets
 *
 * This factory links UObject class types to FUAFAnimNode types.
 */
struct FUAFAnimNodeFactory
{
public:

	// Create a UAFAnimNode instance from the supplied Object
	[[nodiscard]] static UAFANIMNODE_API FUAFAnimNodePtr CreateUAFAnimNodeFromObject(UObject* Object, FUAFAnimGraphUpdateContext& Context);

	// Asset initializer
	using FAssetInitializer = TFunction<FUAFAnimNodePtr(UObject*, FUAFAnimGraphUpdateContext& Context)>;

	// Register a UAF asset with the factory
	// provides a function which will return a FUAFAnimNodePtr from a UObject
	// @return a FTopLevelAssetPath that can be used to unregister the asset type via UnregisterAsset
	static UAFANIMNODE_API FTopLevelAssetPath RegisterAsset(UClass* ObjectClass, FAssetInitializer&& Initializer);

	// Unregister an UAF asset with the factory
	static UAFANIMNODE_API void UnregisterAsset(const FTopLevelAssetPath& InObjectClassPath);
	
	// check if an asset type is registered
	static UAFANIMNODE_API bool IsAssetRegistered(const UClass* InAssetClass);

private:

	struct FInitializerInfo
	{
		FInitializerInfo(FAssetInitializer Initializer)
			: Initializer(Initializer){}

		FInitializerInfo() = default;
		
		FAssetInitializer Initializer;
	};
	
	
	static UAFANIMNODE_API const FInitializerInfo* FindAssetInitializer(const UClass* Class);
	static TMap<FTopLevelAssetPath, FInitializerInfo> AssetInitializers;
};
	
}
