// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConstExprUID.h"

namespace UE::UAF
{
	using FUAFAnimNodeInterfaceId = FConstExprUID;
}
