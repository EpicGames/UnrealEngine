// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	class FUAFAnimNode;

	// Collects anim nodes that need exposure to GC
	class FUAFAnimNodeReferenceCollector : public FGCObject
	{
	public:
		// We use the same pattern as UObject::AddReferencedObjects and cache a function pointer to
		// the function that will add the references. This is done to avoid an extra indirection
		// when the GC runs.
		using AddReferencedObjectsFn = void(*)(FUAFAnimNode*, FReferenceCollector&);

		FUAFAnimNodeReferenceCollector() = default;
		UE_API ~FUAFAnimNodeReferenceCollector();

		// Registers the provided anim node with the GC system
		// Once registered, TUAFAnimNode<T>::AddReferencedObjects will be called on it during GC
		UE_API void RegisterWithGC(FUAFAnimNode* AnimNode, AddReferencedObjectsFn GCFn);

		// Unregisters the provided anim node from the GC system
		UE_API void UnregisterWithGC(FUAFAnimNode* AnimNode);

		// FGCObject implementation
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		UE_API virtual FString GetReferencerName() const override;

	private:
		struct FEntry
		{
			FUAFAnimNode* AnimNode = nullptr;
			AddReferencedObjectsFn GCFn = nullptr;
		};

		// List of anim nodes that contain UObject references
		TArray<FEntry> AnimNodes;
	};
}

#undef UE_API
