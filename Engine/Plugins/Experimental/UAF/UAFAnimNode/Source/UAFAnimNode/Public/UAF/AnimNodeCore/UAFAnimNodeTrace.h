// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RewindDebugger/UAFTrace.h"
#include "StructUtils/StructView.h"

#define UE_API UAFANIMNODE_API

#if UAF_TRACE_ENABLED

struct FUAFAssetInstance;

namespace UE::UAF
{
	class FUAFAnimNode;
	class FUAFAnimGraphUpdateContext;
	struct FUAFAnimOp;

	UE_API void TraceAnimOps(const TArray<FUAFAnimOp*>& AnimOps, const FUAFAssetInstance& AssetInstance, const UObject* OuterObject);
	UE_API void TraceAnimNode(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node);

	UE_API void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, bool Value);
	UE_API void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, double Value);
	UE_API void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, const UObject* Value);
	UE_API void TraceAnimNodeValue(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* Node, FName PropertyName, FStructView Value);
}

#define UAF_TRACE_ANIMOPS(AnimOps, AssetInstance, Object) UE::UAF::TraceAnimOps(AnimOps, AssetInstance, Object);
#define UAF_TRACE_ANIMNODE(UpdateContext, Node) UE::UAF::TraceAnimNode(UpdateContext, Node);
#define UAF_TRACE_ANIMNODE_VALUE(UpdateContext, Node, Name, Value) UE::UAF::TraceAnimNodeValue(UpdateContext, Node, Name, Value);

#else

#define UAF_TRACE_ANIMOPS(AnimOps, AssetInstance, Object)
#define UAF_TRACE_ANIMNODE(UpdateContext, Node)
#define UAF_TRACE_ANIMNODE_VALUE(UpdateContext, Node, Name, Value)

#endif

#undef UE_API
