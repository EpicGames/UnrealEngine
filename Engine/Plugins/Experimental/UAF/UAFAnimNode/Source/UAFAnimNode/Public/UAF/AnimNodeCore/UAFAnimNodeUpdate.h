// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LODPose.h"
#include "UAFAnimNodeTrace.h"

#define UE_API UAFANIMNODE_API

struct FAnimNextGraphLODPose;
struct FAnimNotifyEventReference;
struct FRigUnit_RunAnimNode;
struct FUAFAssetInstance;
class UAbstractSkeletonSetBinding;

namespace UE::UAF
{
	namespace Private
	{
		struct FUAFUpdateNodeEntry;
	}

	class FUAFAnimNode;
	class FUAFAnimNodeReferenceCollector;
	class FUAFAnimNotifyCollector;
	struct FUAFAnimOp;
}

namespace UE::UAF
{
	/**
	 * FUAFAnimGraphUpdateContext
	 *
	 * Contains the necessary machinery for traversal during the update pass.
	 */
	class FUAFAnimGraphUpdateContext
	{
	public:
		UE_NONCOPYABLE(FUAFAnimGraphUpdateContext);	// We disallow copy/move semantics

		UE_API FUAFAnimGraphUpdateContext(UObject* HostObject, FUAFAssetInstance* VariablesOwner, uint64 DebugOuterInstanceId, FUAFAnimNodeReferenceCollector& GCReferences, float DeltaTime);
		UE_API ~FUAFAnimGraphUpdateContext();

		// Returns the current update context in use on this thread or nullptr if none are within our current scope
		[[nodiscard]] UE_API static FUAFAnimGraphUpdateContext* GetCurrentFromTLS();
	
		// Returns the UAF asset that owns the shared variables
		[[nodiscard]] FUAFAssetInstance* GetVariablesOwner();

		// Returns the object that hosts the current graph
		[[nodiscard]] UObject* GetHostObject();

		// Returns a reference collector for anim nodes that need exposure to GC
		[[nodiscard]] FUAFAnimNodeReferenceCollector& GetGCReferences();

		// Returns how much time should the update pass advance by factoring in the current play rate scaling
		[[nodiscard]] float GetDeltaTime() const;

		// Returns how much time should the update pass advance by
		[[nodiscard]] float GetUpdateDeltaTime() const;

		// Returns the currently applied play rate when computing the delta time
		[[nodiscard]] float GetPlayRate() const;

		// Pushes a new play rate scale on top of the stack
		void PushPlayRate(float PlayRate);

		// Pops the most recent play rate scale from the top of the stack
		void PopPlayRate();

#if DO_CHECK
		// Debug counter used to enforce invariants
		[[nodiscard]] uint32 GetUpdateCounter() const;
#endif

#if UAF_TRACE_ENABLED
		uint64 GetOuterDebugInstanceId()
 		{
			 return OuterDebugInstanceId;
		}
#endif
	private:
		void QueueForDestruction(FUAFAnimNode* AnimNode);

		FUAFAnimGraphUpdateContext* PreviousContext = nullptr;

		UObject* HostObject = nullptr;
		FUAFAssetInstance* VariablesOwner = nullptr;

		// A stack of previous play rates
		TArray<float> PreviousPlayRateStack;

		// The currently active play rate
		float CurrentPlayRate = 1.0f;

		// The delta time as currently scaled by the play rate
		float DeltaTime = 0.0f;

		// The unscaled delta time used by the update pass
		float UpdateDeltaTime = 0.0f;

		FUAFAnimNodeReferenceCollector& GCReferences;

		TArray<FUAFAnimNode*> NodesPendingDestruction;

#if DO_CHECK
		// Debug counter used to enforce invariants
		uint32 UpdateCounter = 0;
#endif

#if UAF_TRACE_ENABLED
		uint64 OuterDebugInstanceId;
#endif

		friend UE_API TArray<FUAFAnimOp*> UpdateGraph(FUAFAnimGraphUpdateContext& UpdateContext, FUAFAnimNode& RootNode);
		friend FUAFAnimNode;
	};

	// Updates the specified animation graph and advances time forward
	// This populates a list of AnimOps
	UE_API TArray<FUAFAnimOp*> UpdateGraph(FUAFAnimGraphUpdateContext& UpdateContext, FUAFAnimNode& RootNode);

	/**
	 * FUAFEvaluateValuesArgs
	 *
	 * Contains the necessary machinery for evaluating a list of AnimOps to produce a set of output values.
	 */
	class FUAFEvaluateValuesArgs
	{
	public:
		UE_NONCOPYABLE(FUAFEvaluateValuesArgs);	// We disallow copy/move semantics

		UE_API FUAFEvaluateValuesArgs(UAbstractSkeletonSetBinding& SkeletonSetBinding, FName SkeletonSetName, USkeletalMesh* SkeletalMesh, int32 LOD, const TArray<FUAFAnimOp*>& AnimOps, const FReferencePose& ReferencePose);

		// Returns the list of AnimOps used to produce our output values
		[[nodiscard]] const TArray<FUAFAnimOp*>& GetAnimOps() const;

		// Returns the reference pose used to produce our output values
		[[nodiscard]] const FReferencePose& GetReferencePose() const;

		// Returns the current LOD used to produce our output values
		[[nodiscard]] int32 GetLOD() const;

		// Returns the abstract skeleton set binding asset used to produce our output values
		[[nodiscard]] UAbstractSkeletonSetBinding* GetSkeletonSetBinding() const;

		// Returns the abstract skeleton set name used to produce our output values
		[[nodiscard]] FName GetSkeletonSetName() const;

		// Returns the skeletal mesh used to produce our output values
		[[nodiscard]] USkeletalMesh* GetSkeletalMesh() const;

	private:
		const FReferencePose& ReferencePose;
		const TArray<FUAFAnimOp*>& AnimOps;

		UAbstractSkeletonSetBinding* SkeletonSetBinding = nullptr;
		USkeletalMesh* SkeletalMesh = nullptr;
		FName SkeletonSetName;

		int32 LOD = 0;

		friend FRigUnit_RunAnimNode;
	};

	// Evaluates the specified AnimOps to produce our value bundle (pose, curves, attributes, etc)
	UE_API void EvaluateValues(const FUAFEvaluateValuesArgs& Args, FAnimNextGraphLODPose& OutputValues);

	// Evaluates the specified AnimOps to produce a list of notifies
	UE_API TArray<FAnimNotifyEventReference> EvaluateNotifies(const TArray<FUAFAnimOp*>& AnimOps);

	// Encapsulates a phase synchronization contributor
	struct FUAFSyncContributor
	{
		// TODO
	};

	// Evaluates the specified AnimOps to produce a list of phase synchronization contributors
	UE_API TArray<FUAFSyncContributor> EvaluateSyncContributors(const TArray<FUAFAnimOp*>& AnimOps);

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline FUAFAssetInstance* FUAFAnimGraphUpdateContext::GetVariablesOwner()
	{
		return VariablesOwner;
	}

	inline UObject* FUAFAnimGraphUpdateContext::GetHostObject()
	{
		return HostObject;
	}

	inline FUAFAnimNodeReferenceCollector& FUAFAnimGraphUpdateContext::GetGCReferences()
	{
		return GCReferences;
	}

	inline float FUAFAnimGraphUpdateContext::GetDeltaTime() const
	{
		return DeltaTime;
	}

	inline float FUAFAnimGraphUpdateContext::GetUpdateDeltaTime() const
	{
		return UpdateDeltaTime;
	}

	inline float FUAFAnimGraphUpdateContext::GetPlayRate() const
	{
		return CurrentPlayRate;
	}

#if DO_CHECK
	inline uint32 FUAFAnimGraphUpdateContext::GetUpdateCounter() const
	{
		return UpdateCounter;
	}
#endif

	inline const TArray<FUAFAnimOp*>& FUAFEvaluateValuesArgs::GetAnimOps() const
	{
		return AnimOps;
	}

	inline const FReferencePose& FUAFEvaluateValuesArgs::GetReferencePose() const
	{
		return ReferencePose;
	}

	inline int32 FUAFEvaluateValuesArgs::GetLOD() const
	{
		return LOD;
	}

	inline UAbstractSkeletonSetBinding* FUAFEvaluateValuesArgs::GetSkeletonSetBinding() const
	{
		return SkeletonSetBinding;
	}

	inline FName FUAFEvaluateValuesArgs::GetSkeletonSetName() const
	{
		return SkeletonSetName;
	}

	inline USkeletalMesh* FUAFEvaluateValuesArgs::GetSkeletalMesh() const
	{
		return SkeletalMesh;
	}
}

#undef UE_API
