// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/SubScriptStructOf.h"
#include "UAF/UAFAssetData.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

namespace UE::UAF
{
	
/*
 * The FUAFAnimNodeFactory is used to produce FUAFAnimNode instances from FUAFGraphFactoryAsset structs.
 * This is primarily for compatibility between FUAFAnimNode, and Trait Graph systems, and in future FUAFGraphFactoryAsset should be replaced by FUAFAnimNodeData
 *
 * This factory links UObject class types to FUAFAnimNode types.
 */
struct FUAFGraphFactoryAssetAnimNodeFactory
{
public:

	// Create a UAFAnimNode instance from the supplied Struct
	[[nodiscard]] static UAFANIMNODE_API FUAFAnimNodePtr CreateUAFAnimNodeFromObject(TConstStructView<FUAFGraphFactoryAsset>, FUAFAnimGraphUpdateContext& Context);
	
	// Asset initializer
	using FAssetInitializer = TFunction<FUAFAnimNodePtr(TConstStructView<FUAFGraphFactoryAsset>, FUAFAnimGraphUpdateContext& Context)>;

	// Register a Struct with the factory
	// provides a function which will return a FUAFAnimNodePtr from a FUAFGraphFactoryAsset
	// @return a FTopLevelAssetPath that can be used to unregister the asset type via UnregisterAsset
	static UAFANIMNODE_API FTopLevelAssetPath RegisterStruct(TSubScriptStructOf<FUAFGraphFactoryAsset>, FAssetInitializer&& Initializer);

	// Unregister an UAF asset with the factory
	static UAFANIMNODE_API void UnregisterStruct(const FTopLevelAssetPath& InStructPath);
	
	// check if an asset type is registered
	static UAFANIMNODE_API bool IsStructRegistered(TSubScriptStructOf<FUAFGraphFactoryAsset> InAssetClass);

private:

	struct FInitializerInfo
	{
		FInitializerInfo(FAssetInitializer Initializer)
			: Initializer(Initializer){}

		FInitializerInfo() = default;
		
		FAssetInitializer Initializer;
	};
	
	static UAFANIMNODE_API const FInitializerInfo* FindAssetInitializer(const UStruct* Struct);
	static TMap<FTopLevelAssetPath, FInitializerInfo> AssetInitializers;
};
	
}
