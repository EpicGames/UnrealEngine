// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	class FUAFAnimGraphUpdateContext;
}

namespace UE::UAF
{
	/**
	 * TUAFInlineAnimNode
	 *
	 * Wraps an anim node to allow their usage inline withour incurring an extra heap allocation.
	 * This is suitable when a node knows the type of its child and it can be hardcoded.
	 */
	template <class AnimNodeType>
	class TUAFInlineAnimNode
	{
	public:
		explicit TUAFInlineAnimNode(FUAFAnimGraphUpdateContext& Context);

		// Returns a mutable pointer to the wrapped anim node
		[[nodiscard]] AnimNodeType* Get();

		// Indirection operator to facilitate usage
		[[nodiscard]] AnimNodeType* operator->();
		[[nodiscard]] const AnimNodeType* operator->() const;

	private:
		// The inline anim node we wrap
		AnimNodeType Node;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	template <class AnimNodeType>
	inline TUAFInlineAnimNode<AnimNodeType>::TUAFInlineAnimNode(FUAFAnimGraphUpdateContext& Context)
		: Node(Context)
	{
		// Add a reference we never remove, so ref counted pointers will never delete this inline instance
		Node.AddRef();
	}

	template <class AnimNodeType>
	inline AnimNodeType* TUAFInlineAnimNode<AnimNodeType>::Get()
	{
		return &Node;
	}

	template <class AnimNodeType>
	inline AnimNodeType* TUAFInlineAnimNode<AnimNodeType>::operator->()
	{
		return &Node;
	}

	template <class AnimNodeType>
	inline const AnimNodeType* TUAFInlineAnimNode<AnimNodeType>::operator->() const
	{
		return &Node;
	}
}
