// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	struct FUAFAnimNodeData;
	struct FUAFModifierAnimNodeData;

	/**
	 * FUAFModifierAnimNode
	 *
	 * Base struct for modifier anim node instance data. Modifiers have a single child that they wrap.
	 */
	struct FUAFModifierAnimNode : public FUAFAnimNode
	{
		// FUAFAnimNode impl
		UE_API virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;

		// Returns whether or not this modifier has a valid child
		bool HasChild() const;

		// Returns the modifier child, can be an invalid pointer if no child has been initialized
		const FUAFAnimNodePtr& GetChild() const;

	protected:
		UE_API explicit FUAFModifierAnimNode(FUAFAnimGraphUpdateContext& Context);

		// Initializes the modifier with the specified child
		UE_API void InitializeModifier(FUAFAnimGraphUpdateContext& Context, const FUAFModifierAnimNodeData& ModifierData);
		UE_API void InitializeModifier(FUAFAnimGraphUpdateContext& Context, const TInstancedStruct<FUAFAnimNodeData>& Child);
		UE_API void InitializeModifier(FUAFAnimGraphUpdateContext& Context, const FUAFAnimNodePtr& Child);

		// Sets the modifier child
		void SetChild(const FUAFAnimNodePtr& NewChild);

	private:
		// Hide base implementation in favor of a simpler API
		using FUAFAnimNode::GetChildAt;
		using FUAFAnimNode::GetFirstChild;
		using FUAFAnimNode::SetChildAt;
		using FUAFAnimNode::AddChild;
		using FUAFAnimNode::RemoveChild;
		using FUAFAnimNode::RemoveChildAt;
		using FUAFAnimNode::HasChildren;
		using FUAFAnimNode::GetChildren;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline bool FUAFModifierAnimNode::HasChild() const
	{
		return GetFirstChild().IsValid();
	}

	inline const FUAFAnimNodePtr& FUAFModifierAnimNode::GetChild() const
	{
		return GetFirstChild();
	}

	inline void FUAFModifierAnimNode::SetChild(const FUAFAnimNodePtr& NewChild)
	{
		SetChildAt(0, NewChild);
	}
}

#undef UE_API
