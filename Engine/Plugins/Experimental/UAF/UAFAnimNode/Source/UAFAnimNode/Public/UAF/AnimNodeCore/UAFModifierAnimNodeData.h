// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"

#include "UAFModifierAnimNodeData.generated.h"

namespace UE::UAF
{
	/**
	 * FUAFModifierAnimNodeData
	 *
	 * Base struct for modifier anim node shared data. Modifiers have a single child that they wrap.
	 * Allows for special handling in details panel visualization.
	 */
	USTRUCT()
	struct FUAFModifierAnimNodeData : public FUAFAnimNodeData
	{
		GENERATED_BODY()

		UPROPERTY(EditAnywhere, Category = "Data", meta = (ExcludeBaseStruct, EditCondition = "bShowChildren", EditConditionHides, HideEditConditionToggle))
		TInstancedStruct<FUAFAnimNodeData> Child;

		[[nodiscard]] virtual void* GetInterface(FUAFAnimNodeDataInterfaceId Id);

	private:
#if WITH_EDITORONLY_DATA
		// This flag is used to hide child nodes within the editor details panel under certain authoring scenarios
		UPROPERTY()
		bool bShowChildren = true;

		friend struct FUAFAnimNodeDataEx;
#endif
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline void* FUAFModifierAnimNodeData::GetInterface(FUAFAnimNodeDataInterfaceId Id)
	{
		if (Child.IsValid())
		{
			return Child->GetInterface(Id);
		}

		return nullptr;
	}
}
