// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

#include "UAFTransitionNodeData.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFTransitionNodeData
	 *
	 * Base structure for blend transition shared data
	 */
	USTRUCT(meta = (Hidden))
	struct FUAFTransitionNodeData
	{
		GENERATED_BODY()

		virtual ~FUAFTransitionNodeData() = default;

		[[nodiscard]] UE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr Source, FUAFAnimNodePtr Target) const;
	};
}

#undef UE_API
