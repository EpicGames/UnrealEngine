// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataInterfaceId.h"

namespace UE::UAF
{
	struct IUAFAnimNodeDataHasAsset
	{
		static constexpr FUAFAnimNodeDataInterfaceId InterfaceId = FUAFAnimNodeDataInterfaceId::MakeFromString(TEXT("IUAFAnimNodeDataHasAsset"));

		virtual ~IUAFAnimNodeDataHasAsset() = default;
		[[nodiscard]] virtual UObject* GetAsset() const = 0;
	};
}
