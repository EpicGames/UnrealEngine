// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeInterfaceId.h"

enum ETypeAdvanceAnim : int;

namespace UE::UAF
{
	struct IUAFAnimNodeTimeline
	{
		static constexpr FUAFAnimNodeInterfaceId InterfaceId = FUAFAnimNodeInterfaceId::MakeFromString(TEXT("IUAFAnimNodeTimeline"));

		virtual ~IUAFAnimNodeTimeline() = default;
		[[nodiscard]] virtual float GetCurrentTime() const = 0;
		[[nodiscard]] virtual float GetLength() const = 0;
		[[nodiscard]] virtual bool IsLooping() const = 0;
		virtual void SetCurrentTime(float CurrentTime) = 0;
		[[nodiscard]] virtual ETypeAdvanceAnim GetTimeAdvanceResult() const = 0;
	};
}
