// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeInterfaceId.h"


namespace UE::UAF
{
struct IUAFRootMotionProvider
{
	static constexpr FUAFAnimNodeInterfaceId InterfaceId = FUAFAnimNodeInterfaceId::MakeFromString(TEXT("IUAFRootMotionProvider"));

	virtual ~IUAFRootMotionProvider() = default;

	[[nodiscard]] virtual FTransform ExtractRootMotion(float StartTime, float DeltaTime, bool AllowLooping) const = 0; 
};
}
