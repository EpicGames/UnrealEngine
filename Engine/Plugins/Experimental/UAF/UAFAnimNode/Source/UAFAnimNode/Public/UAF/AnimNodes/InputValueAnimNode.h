// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ReferencePose.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"
#include "UAF/ValueRuntime/PoseValueBundle.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "Variables/AnimNextVariableReference.h"

#include "InputValueAnimNode.generated.h"

#define UE_API UAFANIMNODE_API

struct FAnimNextGraphLODPose;

namespace UE::UAF
{

/**
 * FUAFInputValueAnimOp
 * A leaf AnimOp that pushes a cached value bundle or LOD pose onto the evaluation stack.
 * The cached pointers are set by FUAFInputValueAnimNode during PreUpdate because variable
 * access is only available during the update phase, not during the evaluation phase.
 * If no data is available, an empty bundle (reference pose) is pushed instead.
 */
USTRUCT()
struct FUAFInputValueAnimOp : public FUAFAnimOp
{
	GENERATED_BODY()
	UAF_DECLARE_ANIMOP(FUAFInputValueAnimOp)

	FUAFInputValueAnimOp();

	virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	/**
	 * Cached data set by FUAFInputValueAnimNode::PreUpdate each frame.
	 * Non-owning pointers, only valid for the current evaluation cycle.
	 */
	const FValueBundle* CachedBundle = nullptr;
	const FAnimNextGraphLODPose* CachedLODPose = nullptr;
	const TReferencePose<FDefaultAllocator, FDefaultSetAllocator>* TargetRefPose = nullptr;
	
	/** Converts an FAnimNextGraphLODPose into a FValueBundleStack, with optional skeleton remapping. */
	static void ConvertLODPoseToValueBundle(const FAnimNextGraphLODPose& InputPose,
		const TReferencePose<FDefaultAllocator, FDefaultSetAllocator>* TargetRefPose,
		FPoseValueBundleStack& OutValueBundle);
};

/**
 * FUAFInputValueAnimNodeData
 * Serializable data for the input value anim node.
 */
USTRUCT(DisplayName = "Input Value")
struct UE_API FUAFInputValueAnimNodeData : public FUAFAnimNodeData
{
	GENERATED_BODY()

	/** The graph variable to read the input value bundle from. */
	UPROPERTY(EditAnywhere, Category = "Input Value", meta = (AllowedType = FUAFValueBundle))
	FAnimNextVariableReference Input;

	virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
};

/**
 * FUAFInputValueAnimNode
 * An AnimNode leaf that reads a FUAFValueBundle from a graph variable during PreUpdate and pushes it onto the evaluation stack via FUAFInputValueAnimOp.
 */
struct UE_API FUAFInputValueAnimNode : public FUAFAnimNode
{
	FUAFInputValueAnimNode(FUAFAnimGraphUpdateContext& Context, const FUAFInputValueAnimNodeData& InData);

	// FUAFAnimNode interface
	virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override { return nullptr; }
	virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;

#if UAF_TRACE_ENABLED
	virtual FString GetDebugName() const override;
	virtual UStruct* GetDebugStruct() const override;
#endif

private:
	/** The AnimOp that pushes the cached bundle onto the eval stack. */
	FUAFInputValueAnimOp InputValueOp;

	/** Variable reference to read from. */
	FAnimNextVariableReference VariableReference;
};

} // namespace UE::UAF

#undef UE_API
