// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BindableValue/UAFBindableTypes.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimOps/UAFApplyAdditiveAnimOp.h"
#include "UAF/BlendMask/UAFBlendMask.h"

#include "UAFApplyAdditive.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	// Node to apply an additive animation onto a base animation
	// Optionally can use a blend mask for partial application of the additive
	USTRUCT(DisplayName = "Apply Additive")
	struct FUAFApplyAdditiveData : public FUAFAnimNodeData
	{
		GENERATED_BODY()
		
		// Node providing the base animation
		UPROPERTY(EditAnywhere, Category = "NodeData")
		TInstancedStruct<FUAFAnimNodeData> BaseNode;

		// Node providing the additive animation
		UPROPERTY(EditAnywhere, Category = "NodeData")
		TInstancedStruct<FUAFAnimNodeData> AdditiveNode;
		
		// How much of the additive to apply to the base
		UPROPERTY(EditAnywhere, Category = "NodeData")
		FBindableFloat AdditiveWeight = 1.0f;
		
		// Optional blend mask to mask the additive with 
		UPROPERTY(EditAnywhere, Category = "NodeData")
		TObjectPtr<UUAFBlendMask> BlendMask = nullptr;
		
		UE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
	};
	
	
	class FUAFApplyAdditiveNode : public FUAFAnimNode
	{
	public:
		UE_API explicit FUAFApplyAdditiveNode(FUAFAnimGraphUpdateContext& Context, const FUAFApplyAdditiveData& InData);
		
		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
		UE_API virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;
		UE_API static void AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector);
		
#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
#endif
	
	private:
		static constexpr int32 BaseChildIndex = 0;
		static constexpr int32 AdditiveChildIndex = 1;
		
		// Ptr to the nodes data - used to retrieve properties and evaluate bound values 
		const FUAFApplyAdditiveData& Data;
		// AnimOp for this node - performs the actual animation operations
		FUAFApplyAdditiveAnimOp AdditiveAnimOp;
	};
}

#undef UE_API 
