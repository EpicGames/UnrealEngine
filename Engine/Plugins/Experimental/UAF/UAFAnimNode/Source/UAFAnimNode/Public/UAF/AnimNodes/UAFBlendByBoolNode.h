// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataEx.h"
#include "BindableValue/UAFBindableTypes.h"
#include "UAF/AnimNodes/UAFBlendStack.h"

#include "UAFBlendByBoolNode.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	UENUM()
	enum class EBlendByBoolEvaluationFrequency // todo, since this is a common pattern this enum should probably be shared
	{
		// Create a Node Instance of the appropriate child at creation time
		OnCreate,
		// Check the input bool every frame, and when its value changes, transition to the appropriate child node
		OnUpdate
	};

	// BlendByBool Anim Node selects between two child nodes based on a bool
	USTRUCT(DisplayName = "Blend By Bool")
	struct FUAFBlendByBoolNodeData : public FUAFAnimNodeData
	{
		GENERATED_BODY()

		/**
		 * Bool value used to select between TrueNode and FalseNode.
		 * Bind to a UAF bool variable at runtime by setting the Binding field.
		 */
		UPROPERTY(EditAnywhere, Category = "NodeData")
		FBindableBool BoolValue;

		// Evaluation Frequency: should we select the appropriate child once at creation time, or every frame.
		UPROPERTY(EditAnywhere, Category = "NodeData")
		EBlendByBoolEvaluationFrequency EvaluationFrequency = EBlendByBoolEvaluationFrequency::OnUpdate;

		// Child Node to instantiate if input bool is true
		UPROPERTY(EditAnywhere, Category = "NodeData")
		FUAFAnimNodeDataEx TrueNode;

		// Child Node to instantiate if input bool is false
		UPROPERTY(EditAnywhere, Category = "NodeData")
		FUAFAnimNodeDataEx FalseNode;

		// Transition and blend data for transitioning between child nodes
		UPROPERTY(EditAnywhere, Category = "NodeData")
		TInstancedStruct<FUAFTransitionNodeData> Transition;

		// FUAFAnimNodeData impl
		UE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
	};

	// BlendByBool Instance
	struct FUAFBlendByBoolNode : public FUAFBlendStack
	{
		UE_API explicit FUAFBlendByBoolNode(FUAFAnimGraphUpdateContext& Context, const FUAFBlendByBoolNodeData* InData);

		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
		
	#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
    #endif

	private:
		const FUAFBlendByBoolNodeData* Data = nullptr;
		bool bCurrentValue = false;
	};
}

#undef UE_API
