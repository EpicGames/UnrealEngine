// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFSimpleTransition.h"
#include "UAF/AnimOps/UAFBlendTwoPerValueAnimOp.h"

#include "UAFBlendProfileTransition.generated.h"

#define UE_API UAFANIMNODE_API

class UUAFBlendProfile;

namespace UE::UAF
{
	USTRUCT(DisplayName = "Blend Profile Transition")
	struct FUAFBlendProfileTransitionData : public FUAFSimpleTransitionData
	{
		GENERATED_BODY()

		UPROPERTY(EditAnywhere, Category = Blending)
		TObjectPtr<UUAFBlendProfile> BlendProfile;

		// FUAFTransitionNodeData impl
		virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr Source, FUAFAnimNodePtr Target) const override;
	};

	class FUAFBlendProfileTransition : public FUAFTimedTransition
	{
	public:
		FUAFBlendProfileTransition() = delete;
		UE_API FUAFBlendProfileTransition(
			FUAFAnimGraphUpdateContext& Context,
			FUAFAnimNodePtr InSource,
			FUAFAnimNodePtr InTarget,
			TNonNullPtr<UUAFBlendProfile> InBlendProfile,
			float InBaseDuration,
			EAlphaBlendOption InBlendOption);

		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;

#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
#endif
		
	protected:
		FUAFBlendTwoPerValueAnimOp BlendAnimOp;
	};
}

#undef UE_API