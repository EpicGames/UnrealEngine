// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimNodeCore/IUAFTransitionContainerNode.h"
#include "UAF/AnimNodeCore/UAFTransitionNodeData.h"

namespace UE::UAF
{

// FUAFBlend
// Helper node for handling Transitions
// - Handles Transition Pruning
// - Ensures there is a single child
// - Generates a T-Pose if that child is null
//
// Users can either:
// - Subclass this node, and implement the Update function
// - Compose a node using this node as a child

	
class FUAFBlendStack : public FUAFAnimNode, public IUAFTransitionContainerNode
{
public:
	UAFANIMNODE_API FUAFBlendStack(FUAFAnimGraphUpdateContext& Context);

	// FAnimNode
	UAFANIMNODE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& Context) override;
	UAFANIMNODE_API virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;

	// IUAFTransitionContainerNode interface
	UAFANIMNODE_API virtual void NotifyTransitionComplete(const IUAFTransitionNode& TransitionNode) override;

	// Trigger blend to new node instance
	UAFANIMNODE_API void TransitionTo(FUAFAnimGraphUpdateContext& Context, FUAFAnimNode* TargetNodeData, const FUAFTransitionNodeData* TransitionData = nullptr);

	// returns true if the BlendStack has a valid child node
	bool HasValidChild() const;
		
#if UAF_TRACE_ENABLED
		virtual FString GetDebugName() const override;
		virtual UStruct* GetDebugStruct() const override;
#endif

protected:
	const FUAFAnimNodePtr& GetChild() const;
	FUAFAnimNodePtr ReleaseChild();
	void SetChild(const FUAFAnimNodePtr& NewChild);
	
	// if there is no valid child, add a Reference Pose Op, if there is a valid child clear Op
	UAFANIMNODE_API void ValidateSingleChild();

private:
	// if the contained blend finished last update, replace it with it's target node
	UAFANIMNODE_API void PruneTransitions();

	
	// force subclasses to use GetChild/SetChild/ReleaseChild
	using FUAFAnimNode::GetChildAt;
	using FUAFAnimNode::GetFirstChild;
	using FUAFAnimNode::SetChildAt;
	using FUAFAnimNode::AddChild;
	
	bool bTransitionCompleted = false;
};

inline bool FUAFBlendStack::HasValidChild() const
{
	return GetFirstChild() != nullptr;
}
	
inline const FUAFAnimNodePtr& FUAFBlendStack::GetChild() const
{
	return GetFirstChild();
}
		
inline FUAFAnimNodePtr FUAFBlendStack::ReleaseChild()
{
	FUAFAnimNodePtr Child = GetFirstChild();
	SetChildAt(0, nullptr);
	return Child;
}
	
inline void FUAFBlendStack::SetChild(const FUAFAnimNodePtr& NewChild)
{
	ensure(!GetChild().IsValid());
	SetChildAt(0, NewChild);
}

}
