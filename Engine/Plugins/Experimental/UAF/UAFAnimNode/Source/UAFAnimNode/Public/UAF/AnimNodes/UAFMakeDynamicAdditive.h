// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BindableValue/UAFBindableTypes.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimOps/UAFMakeDynamicAdditiveAnimOp.h"

#include "UAFMakeDynamicAdditive.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	// Node to generate an additive animation from Base and Source poses.
	// The output is an additive pose. Additive =  (Source - Base).
	// Generates an additive identity in case of a bad content setup.
	USTRUCT(DisplayName = "Make Dynamic Additive")
	struct FUAFMakeDynamicAdditiveData : public FUAFAnimNodeData
	{
		GENERATED_BODY()

		// Node providing the base (reference) animation
		UPROPERTY(EditAnywhere, Category = "NodeData")
		TInstancedStruct<FUAFAnimNodeData> BaseNode;

		// Node providing the Source animation to turn into an additive
		UPROPERTY(EditAnywhere, Category = "NodeData")
		TInstancedStruct<FUAFAnimNodeData> SourceNode;

		// Whether to compute the additive delta in mesh space (rotation only).
		UPROPERTY(EditAnywhere, Category = "NodeData")
		bool bMeshSpaceAdditive = false;

		UE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
	};

	class FUAFMakeDynamicAdditiveNode : public FUAFAnimNode
	{
	public:
		UE_API explicit FUAFMakeDynamicAdditiveNode(FUAFAnimGraphUpdateContext& Context, const FUAFMakeDynamicAdditiveData& InData);

		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
		UE_API virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;
		UE_API static void AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector);

#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
#endif

	private:
		static constexpr int32 BaseChildIndex = 0;
		static constexpr int32 SourceChildIndex = 1;

		const FUAFMakeDynamicAdditiveData* Data = nullptr;
		FUAFMakeDynamicAdditiveAnimOp MakeDynamicAdditiveAnimOp;
	};
}

#undef UE_API
