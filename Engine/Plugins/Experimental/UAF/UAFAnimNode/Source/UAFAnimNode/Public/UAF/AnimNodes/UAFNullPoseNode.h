// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	// Dummy Anim node, produces a T Pose
	// For use by other nodes when their child data is not set or for other failure cases.

	class FUAFNullPoseNode : public FUAFAnimNode
	{
	public:
		UE_API explicit FUAFNullPoseNode(FUAFAnimGraphUpdateContext& Context);

		virtual void* GetInterface(FUAFAnimNodeInterfaceId Id)
		{
			return nullptr;
		};

#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
#endif
	};
}

#undef UE_API
