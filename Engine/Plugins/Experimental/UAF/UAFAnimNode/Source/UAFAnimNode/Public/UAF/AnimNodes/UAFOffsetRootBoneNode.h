// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BindableValue/UAFBindableTypes.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNode.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"
#include "UAF/AnimOps/UAFOffsetRootBoneAnimOp.h"
#include "UAFOffsetRootBoneNode.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{

/**
 * Adds a procedural offset to the root bone coming from incoming root motion.
 * It can optionally fade this out over time (controlled by EUAFOffsetRootBoneNodeMode).
 *
 * This allows a code-controlled character to still be influenced by root motion,
 * e.g. during plant turns. It can lead to less foot sliding in a motion matching setup.
 */
USTRUCT(DisplayName = "Offset Root Bone")
struct FUAFOffsetRootBoneNodeData : public FUAFModifierAnimNodeData
{
	GENERATED_BODY()

	// Current strength of the skeletal control
	UPROPERTY(EditAnywhere, Category = Alpha)
	FBindableFloat Alpha = 1.0f;

	// The translation offset behavior mode
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableEnum TranslationMode = FBindableEnum(EUAFOffsetRootBoneNodeMode::Interpolate);

	// The rotation offset behavior mode
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableEnum RotationMode = FBindableEnum(EUAFOffsetRootBoneNodeMode::Interpolate);

	// Controls how fast the translation offset is blended out
	// Values closer to 0 make it faster
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat TranslationSmoothingTime = 0.1f;

	// Controls how fast the rotation offset is blended out
	// Values closer to 0 make it faster
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat RotationSmoothingTime = 0.2f;

	// How much the offset can deviate from the mesh component's translation in units
	// Values lower than 0 disable this limit
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat MaxTranslationError = -1.0f;

	// How much the offset can deviate from the mesh component's rotation in degrees
	// Values lower than 0 disable this limit
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat MaxRotationErrorDegrees = -1.0f;

	// Whether to limit the offset's translation interpolation speed to the velocity on the incoming motion
	// Enabling this prevents the offset sliding when there's little to no translation speed
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline))
	bool bClampToTranslationVelocity = false;

	// Whether to limit the offset's rotation interpolation speed to the velocity on the incoming motion
	// Enabling this prevents the offset sliding when there's little to no rotation speed
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline))
	bool bClampToRotationVelocity = false;

	// How much the offset can blend out, relative to the incoming translation speed
	// i.e. If root motion is moving at 400cm/s, at 0.5, the offset can blend out at 200cm/s
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = bClampToTranslationVelocity, Inline))
	float TranslationSpeedRatio = 0.5f;

	// How much the offset can blend out, relative to the incoming rotation speed
	// i.e. If root motion is rotating at 90 degrees/s, at 0.5, the offset can blend out at 45 degree/s
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditCondition = bClampToRotationVelocity, Inline))
	float RotationSpeedRatio = 0.5f;

	// This has to be the mesh component transform in world space
	UPROPERTY(EditAnywhere, Category = Evaluation)
	FBindableTransform MeshComponentTransformWorld;

	// Signals whether the offset root bone translation should be projected onto a projected ground plane
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableBool bOnGround = true;

	// Surface normal that the current animation was authored for. Used with the bOnGround flag.
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableVector AnimatedGroundNormal = FVector(0, 0, 1);

	// Distance threshold used to detect teleports
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline))
	float TeleportDistanceThreshold = 300.0f;

	// If to reset the root bone offset on teleport
	UPROPERTY(EditAnywhere, Category = Settings, meta = (Inline))
	bool bResetOnTeleport = false;

	virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
};

class FUAFOffsetRootBoneNode : public FUAFModifierAnimNode
{
public:
	UE_API FUAFOffsetRootBoneNode(FUAFAnimGraphUpdateContext& Context, const FUAFOffsetRootBoneNodeData& InData);

#if UAF_TRACE_ENABLED
	virtual FString GetDebugName() const override;
	virtual UStruct* GetDebugStruct() const override;
#endif

protected:
	virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;

	const FUAFOffsetRootBoneNodeData* Data;

	FUAFOffsetRootBoneAnimOp OffsetRootBoneAnimOp;
};

} // namespace UE::UAF

#undef UE_API
