// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNode.h"
#include "BindableValue/UAFBindableTypes.h"

#include "UAFScalePlayRate.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	// Anim node shared data for scaling the play rate of a child
	USTRUCT(DisplayName = "Scale Play Rate")
	struct FUAFScalePlayRateData : public FUAFModifierAnimNodeData
	{
		GENERATED_BODY()

		UPROPERTY(EditAnywhere, Category = "Data")
		FBindableFloat PlayRateMultiplier = 1.0f;

		// FUAFAnimNodeData impl
		UE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
	};

	// Anim node instance for scaling the play rate of a child
	struct FUAFScalePlayRate : public FUAFModifierAnimNode
	{
		UE_API FUAFScalePlayRate(FUAFAnimGraphUpdateContext& Context, const FUAFScalePlayRateData& InData);

		// Sets the play rate
		// This clears any existing bindings
		UE_API void SetPlayRate(float PlayRate);

		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& Context) override;
		UE_API virtual void PostUpdate(FUAFAnimGraphUpdateContext& Context) override;

#if UAF_TRACE_ENABLED
		virtual FString GetDebugName() const override;
		virtual UStruct* GetDebugStruct() const override;
#endif
	private:
		FBindableFloat PlayRateMultiplier = 1.0f;
	};
}

#undef UE_API
