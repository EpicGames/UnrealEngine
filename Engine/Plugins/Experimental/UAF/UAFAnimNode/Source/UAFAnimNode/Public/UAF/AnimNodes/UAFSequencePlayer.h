// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IUAFRootMotionProvider.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodes/IUAFAnimNodeDataHasAsset.h"
#include "UAF/AnimNodes/IUAFAnimNodeTimeline.h"
#include "UAF/AnimOps/UAFDecompressAnimSequenceAnimOp.h"
#include "Animation/AnimSequence.h"
#include "Factory/AnimAssetParams.h" // for EAnimAssetLoopMode

#include "UAFSequencePlayer.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	// Anim node shared data for playing an AnimSequence
	USTRUCT(DisplayName = "Sequence Player")
	struct FUAFSequencePlayerData : public FUAFAnimNodeData
#if CPP
	, public IUAFAnimNodeDataHasAsset
#endif
	{
		GENERATED_BODY()

		UPROPERTY(EditAnywhere, Category="Data")
		TObjectPtr<UAnimSequence> Sequence;

		/** Whether to use the asset's default looping behavior or force an override. */
		UPROPERTY(EditAnywhere, Category = "Data")
		EAnimAssetLoopMode LoopMode = EAnimAssetLoopMode::Auto;

		UPROPERTY(EditAnywhere, Category = "Data")
		float StartTime = 0.0f;

		// FUAFAnimNodeData impl
		UE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
		UE_API virtual void* GetInterface(FUAFAnimNodeDataInterfaceId Id) override;

		// IAnimNodeDataHasAsset impl
		UE_API virtual UObject* GetAsset() const override { return Sequence; }
	};

	// Anim node instance for playing an AnimSequence 
	struct FUAFSequencePlayer : public FUAFAnimNode, public IUAFAnimNodeTimeline, public IUAFRootMotionProvider
	{
		UE_API explicit FUAFSequencePlayer(FUAFAnimGraphUpdateContext& Context, const FUAFSequencePlayerData* InData);
		UE_API explicit FUAFSequencePlayer(FUAFAnimGraphUpdateContext& Context, const UAnimSequence* Sequence, EAnimAssetLoopMode LoopMode = EAnimAssetLoopMode::Auto);

		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
		UE_API virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;
		UE_API static void AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector);

#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
#endif
		
		// IUAFAnimNodeTimeline
		UE_API virtual float GetCurrentTime() const override;
		UE_API virtual void SetCurrentTime(float InCurrentTime) override;
		UE_API virtual bool IsLooping() const override;
		UE_API virtual float GetLength() const override;
		UE_API virtual ETypeAdvanceAnim GetTimeAdvanceResult() const override;

		// IUAFRootMotionProvider
		UE_API virtual FTransform ExtractRootMotion(float StartTime, float DeltaTime, bool AllowLooping) const override;

	private:
		FUAFDecompressAnimSequenceAnimOp DecompressAnimOp;
	};
}

#undef UE_API
