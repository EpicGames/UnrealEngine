// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "UAFTimedTransition.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodeCore/IUAFTransitionNode.h"
#include "UAF/AnimNodeCore/IUAFTransitionContainerNode.h"
#include "UAF/AnimNodeCore/UAFTransitionNodeData.h"
#include "UAF/AnimOps/UAFBlendTwoAnimOp.h"

#include "UAFSimpleTransition.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFSimpleTransitionData
	 *
	 * Shared data for a simple blend transition.
	 */
	USTRUCT(DisplayName = "Simple Transition")
	struct FUAFSimpleTransitionData : public FUAFTransitionNodeData
	{
		GENERATED_BODY()

		UPROPERTY(EditAnywhere, Category = Blending)
		float Duration = 0.1f;

		UPROPERTY(EditAnywhere, Category = Blending)
		EAlphaBlendOption BlendOption = UE::Anim::DefaultBlendOption;

		// FUAFTransitionNodeData impl
		virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr Source, FUAFAnimNodePtr Target) const override;
	};

	/**
	 * FUAFSimpleTransition
	 *
	 * Instance data for a simple blend transition
	 */
	class FUAFSimpleTransition : public FUAFTimedTransition
	{
	public:
		UE_API FUAFSimpleTransition(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr InSource, FUAFAnimNodePtr InTarget, float InDuration, EAlphaBlendOption InBlendOption);

		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
		
#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
#endif
		FUAFBlendTwoAnimOp BlendAnimOp;
	};
}

#undef UE_API
