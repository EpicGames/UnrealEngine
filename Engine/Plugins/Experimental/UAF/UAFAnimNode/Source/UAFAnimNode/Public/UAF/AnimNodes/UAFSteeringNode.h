// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BindableValue/UAFBindableTypes.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNode.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"
#include "UAF/AnimOps/UAFSteeringAnimOp.h"
#include "UAFSteeringNode.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{

USTRUCT(DisplayName = "Steering")
struct FUAFSteeringNodeData : public FUAFModifierAnimNodeData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat Alpha = 1.0f;

	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableQuat TargetOrientation;

	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableTransform RootBoneTransform;

	// The number of seconds in the future before we should reach the TargetOrientation when play animations with no root motion rotation
	UPROPERTY(EditAnywhere, Category=Settings)
	FBindableFloat ProceduralTargetTime = 0.2f;

	// The number of seconds in the future before we should reach the TargetOrientation when playing animations with root motion rotation
	UPROPERTY(EditAnywhere, Category=Settings)
	FBindableFloat AnimatedTargetTime = 0.2f;

	UPROPERTY(EditAnywhere, Category=Settings)
	FBindableFloat RootMotionAngleThresholdDegrees = 1.0f;

	// Will clamp the scaling ratio applied to above this threashold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat MinScaleRatio = 0.5f;

	// Will clamp the scaling ratio applied to below this threashold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat MaxScaleRatio = 1.5f;

	// below this movement speed (based on the root motion in the animation) disable steering coming from the additive spring based correction
	UPROPERTY(EditAnywhere, Category = Settings)
	FBindableFloat DisableAdditiveBelowSpeed = -1.0f;

	// below this movement speed (based on the root motion in the animation) disable steering completely (both scaling and additive)
	UPROPERTY(EditAnywhere, Category=Settings)
	FBindableFloat DisableSteeringBelowSpeed = 1.0f;

	// FUAFTransitionNodeData impl
	virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
};

class FUAFSteeringNode : public FUAFModifierAnimNode
{
public:
	UE_API FUAFSteeringNode(FUAFAnimGraphUpdateContext& Context, const FUAFSteeringNodeData& InData);

#if UAF_TRACE_ENABLED
	virtual FString GetDebugName() const override;
	virtual UStruct* GetDebugStruct() const override;
#endif

protected:
	virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
	virtual void PostUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;

	const FUAFSteeringNodeData* Data;

	FUAFSteeringAnimOp SteeringAnimOp;
};
}

#undef UE_API