// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodeCore/IUAFTransitionNode.h"
#include "UAF/AnimNodeCore/IUAFTransitionContainerNode.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFTimedTransition
	 */
	class FUAFTimedTransition : public FUAFAnimNode, public IUAFTransitionNode, public IUAFTransitionContainerNode 
	{
	public:
		UE_API FUAFTimedTransition(FUAFAnimGraphUpdateContext& Context, FUAFAnimNodePtr InSource, FUAFAnimNodePtr InTarget, float InDuration, EAlphaBlendOption InBlendOption);

		// ITransitionInstance impl
		virtual const FUAFAnimNodePtr& GetSource() const override;
		virtual const FUAFAnimNodePtr& GetTarget() const override;
		UE_API virtual FUAFAnimNodePtr ReleaseSource() override;
		UE_API virtual FUAFAnimNodePtr ReleaseTarget() override;
		virtual bool IsComplete() const override;

		// ITransitionContainer impl
		UE_API virtual void NotifyTransitionComplete(const IUAFTransitionNode& TransitionNode) override;

		// FUAFAnimNode impl
		UE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
		UE_API virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;
		
#if UAF_TRACE_ENABLED
		UE_API virtual FString GetDebugName() const override;
		UE_API virtual UStruct* GetDebugStruct() const override;
#endif

	protected:
		static constexpr int32 SourceChildIndex = 0;
		static constexpr int32 TargetChildIndex = 1;

		float Duration = 0.1f;
		EAlphaBlendOption BlendOption = UE::Anim::DefaultBlendOption;

		float TimeRemaining = 0.0f;
		bool bNestedTransitionComplete = false;
	};
	
	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline const FUAFAnimNodePtr& FUAFTimedTransition::GetSource() const
	{
		return GetChildAt(SourceChildIndex);
	}

	inline const FUAFAnimNodePtr& FUAFTimedTransition::GetTarget() const
	{
		return GetChildAt(TargetChildIndex);
	}

	inline bool FUAFTimedTransition::IsComplete() const
	{
		return TimeRemaining <= 0.0f;
	}
}

#undef UE_API
