// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UAFAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

// Disable the debug pointer to our AnimOp owner in Test and Shipping builds
#define UAF_ENABLE_DEBUG_ANIMOP_OWNER (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

// Declares various implementation details for an animation operation
#define UAF_DECLARE_ANIMOP(AnimOpName) \
	virtual const UScriptStruct* GetStruct() const override { return AnimOpName::StaticStruct(); }

namespace UE::UAF
{
	class FUAFAnimNode;
	class FUAFAnimOpSyncEvaluator;
	class FUAFAnimOpNotifyEvaluator;
	class FUAFAnimOpValueEvaluator;
}

namespace UE::UAF
{
	/**
	 * FUAFAnimOp
	 *
	 * An anim operation instance.
	 * AnimOps are used to produce and manipulate: animated values, notifies, and sync contributors.
	 */
	USTRUCT()
	struct FUAFAnimOp
	{
		GENERATED_BODY()

		// ForceInit constructor for TCppStructOps that creates an invalid AnimOp
		explicit FUAFAnimOp(EForceInit);
		UE_API virtual ~FUAFAnimOp();

		// Returns the derived type for the AnimOp
		virtual const UScriptStruct* GetStruct() const;

		// Returns the number of inputs this AnimOp consumes
		[[nodiscard]] uint8 GetNumInputs() const;

		// Whether or not this op implements EvaluateValues(..)
		[[nodiscard]] bool HasEvaluateValues() const;

		// Whether or not this op implements EvaluateNotifies(..)
		[[nodiscard]] bool HasEvaluateNotifies() const;

		// Whether or not this op implements EvaluateSynchronization(..)
		[[nodiscard]] bool HasEvaluateSynchronization() const;

		// Entry point for AnimOps that manipulate animated values
		// Values are evaluated first
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator);

		// Entry point for AnimOps that manipulate notifies
		// Notifies are evaluated second
		UE_API virtual void EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator);

		// Entry point for AnimOps that manipulate synchronization contributors
		// Synchronization is evaluated third
		UE_API virtual void EvaluateSynchronization(FUAFAnimOpSyncEvaluator& Evaluator);

		// Gets the debug owner for this AnimOp
		// Can be nullptr
		[[nodiscard]] const FUAFAnimNode* GetDebugOwner() const;

		// Sets the debug owner for this AnimOp
		void SetDebugOwner(const FUAFAnimNode* Owner);

	protected:
		// Initialization function should be called within the constructor of derived types
		template<class AnimOpType>
		void InitializeAs();

		UE_API FUAFAnimOp(uint8 NumInputs);

	private:
#if UAF_ENABLE_DEBUG_ANIMOP_OWNER
		// The optional anim node that owns this AnimOp
		const FUAFAnimNode* DebugOwner = nullptr;
#endif

		// How many inputs does this AnimOp consume
		UPROPERTY(VisibleAnywhere, Category = Properties)
		uint8 NumInputs = 0;

		// Whether or not this op implements EvaluateValues(..)
		bool bHasEvaluateValues : 1 = false;

		// Whether or not this op implements EvaluateNotifies(..)
		bool bHasEvaluateNotifies : 1 = false;

		// Whether or not this op implements EvaluateSynchronization(..)
		bool bHasEvaluateSynchronization : 1 = false;

#if DO_CHECK
		// Whether or not InitializeAs<T>(..) has been called
		bool bIsInitialized : 1 = false;
#endif
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline FUAFAnimOp::FUAFAnimOp(EForceInit)
		: NumInputs(0)
	{
#if DO_CHECK
		bIsInitialized = true;
#endif
	}

	inline const UScriptStruct* FUAFAnimOp::GetStruct() const
	{
		return FUAFAnimOp::StaticStruct();
	}

	inline uint8 FUAFAnimOp::GetNumInputs() const
	{
		return NumInputs;
	}

	inline bool FUAFAnimOp::HasEvaluateValues() const
	{
		return bHasEvaluateValues;
	}

	inline bool FUAFAnimOp::HasEvaluateNotifies() const
	{
		return bHasEvaluateNotifies;
	}

	inline bool FUAFAnimOp::HasEvaluateSynchronization() const
	{
		return bHasEvaluateSynchronization;
	}

	template<class AnimOpType>
	inline void FUAFAnimOp::InitializeAs()
	{
		bHasEvaluateValues = !std::is_same_v<decltype(&AnimOpType::EvaluateValues), decltype(&FUAFAnimOp::EvaluateValues)>;
		bHasEvaluateNotifies = !std::is_same_v<decltype(&AnimOpType::EvaluateNotifies), decltype(&FUAFAnimOp::EvaluateNotifies)>;
		bHasEvaluateSynchronization = !std::is_same_v<decltype(&AnimOpType::EvaluateSynchronization), decltype(&FUAFAnimOp::EvaluateSynchronization)>;

#if DO_CHECK
		bIsInitialized = true;
#endif
	}

	inline const FUAFAnimNode* FUAFAnimOp::GetDebugOwner() const
	{
#if UAF_ENABLE_DEBUG_ANIMOP_OWNER
		return DebugOwner;
#else
		return nullptr;
#endif
	}

	inline void FUAFAnimOp::SetDebugOwner(const FUAFAnimNode* Owner)
	{
#if UAF_ENABLE_DEBUG_ANIMOP_OWNER
		DebugOwner = Owner;
#endif
	}
}

template<>
struct TStructOpsTypeTraits<UE::UAF::FUAFAnimOp> : TStructOpsTypeTraitsBase2<UE::UAF::FUAFAnimOp>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};

#undef UE_API
