// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOpListBase.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFAnimOpArrayView
	 *
	 * An array view of AnimOps.
	 * AnimOps are evaluated in the ascending order.
	 */
	struct FUAFAnimOpArrayView : public FUAFAnimOpListBase
	{
		FUAFAnimOpArrayView();

		// Creates an array view from a C-style array
		template<size_t Count> static FUAFAnimOpArrayView Make(FUAFAnimOp*(&AnimOps)[Count]);
		template<size_t Count> static FUAFAnimOpArrayView Make(FUAFAnimOp* (&AnimOps)[Count], int32 Num);

		// Creates an array view from a TArray with any allocator
		template<class AllocatorType> static FUAFAnimOpArrayView Make(const TArray<FUAFAnimOp*, AllocatorType>& AnimOps);
		template<class AllocatorType> static FUAFAnimOpArrayView Make(const TArray<FUAFAnimOp*, AllocatorType>& AnimOps, int32 Num);

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;
		UE_API virtual void EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator) override;
		UE_API virtual void EvaluateSynchronization(FUAFAnimOpSyncEvaluator& Evaluator) override;

	protected:
		// FUAFAnimOpListBase impl
		UE_API virtual void GetAnimOps(TAdderRef<const FUAFAnimOp*> OutAnimOps) const override;

		// An array view of AnimOps, evaluated in ascending order
		TArrayView<FUAFAnimOp*> AnimOps;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline FUAFAnimOpArrayView::FUAFAnimOpArrayView()
		: FUAFAnimOpListBase()
	{
		InitializeAs<FUAFAnimOpArrayView>();
	}

	template<size_t Count>
	inline FUAFAnimOpArrayView FUAFAnimOpArrayView::Make(FUAFAnimOp* (&AnimOps)[Count])
	{
		FUAFAnimOpArrayView Result;
		Result.AnimOps = MakeArrayView(AnimOps, Count);
		return Result;
	}

	template<size_t Count>
	inline FUAFAnimOpArrayView FUAFAnimOpArrayView::Make(FUAFAnimOp* (&AnimOps)[Count], int32 Num)
	{
		checkf(Num <= Count, TEXT("Cannot create a view that extends past the end of the input array."));

		FUAFAnimOpArrayView Result;
		Result.AnimOps = MakeArrayView(AnimOps, Num);
		return Result;
	}

	template<class AllocatorType>
	inline FUAFAnimOpArrayView FUAFAnimOpArrayView::Make(const TArray<FUAFAnimOp*, AllocatorType>& AnimOps)
	{
		FUAFAnimOpArrayView Result;
		Result.AnimOps = MakeArrayView(AnimOps);
		return Result;
	}

	template<class AllocatorType>
	inline FUAFAnimOpArrayView FUAFAnimOpArrayView::Make(const TArray<FUAFAnimOp*, AllocatorType>& AnimOps, int32 Num)
	{
		checkf(Num <= AnimOps.Num(), TEXT("Cannot create a view that extends past the end of the input array."));

		FUAFAnimOpArrayView Result;
		Result.AnimOps = MakeArrayView(AnimOps.GetData(), Num);
		return Result;
	}
}

#undef UE_API
