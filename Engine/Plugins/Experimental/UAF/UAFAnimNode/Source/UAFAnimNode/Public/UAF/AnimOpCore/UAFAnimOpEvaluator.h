// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/UAFStack.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/*
	 * FUAFAnimOpEvaluator
	 *
	 * Base class for AnimOp evaluators that provides basic functionality.
	 */
	class FUAFAnimOpEvaluator
	{
	public:
		UE_NONCOPYABLE(FUAFAnimOpEvaluator);	// We disallow copy/move semantics

		// Returns a user defined stack and creates it if it doesn't exist yet
		template<typename ValueType>
		[[nodiscard]] TUAFStack<ValueType>& GetOrCreateStack(const FUAFStackName& StackName);

		// Returns a pointer to a user defined stack, can be nullptr if it doesn't exist yet
		template<typename ValueType>
		[[nodiscard]] TUAFStack<ValueType>* FindStack(const FUAFStackName& StackName);

		// Returns a pointer to a user defined stack, can be nullptr if it doesn't exist yet
		template<typename ValueType>
		[[nodiscard]] const TUAFStack<ValueType>* FindStack(const FUAFStackName& StackName) const;

	protected:
		UE_API FUAFAnimOpEvaluator();
		UE_API ~FUAFAnimOpEvaluator();

	private:
		// User defined stacks that AnimOps can use
		TMap<FName, TUniquePtr<FUAFStack>> UserDefinedStacks;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	template<typename ValueType>
	inline TUAFStack<ValueType>& FUAFAnimOpEvaluator::GetOrCreateStack(const FUAFStackName& StackName)
	{
		if (TUniquePtr<FUAFStack>* Stack = UserDefinedStacks.FindByHash(StackName.GetNameHash(), StackName.GetName()))
		{
			checkf(Stack->Get()->GetTypeID() == GetStackTypeID<ValueType>(), TEXT("Type mismatch! This stack is being queried with a different type than it was created with."));
			return *static_cast<TUAFStack<ValueType>*>(Stack->Get());
		}

		TUniquePtr<FUAFStack> Stack = MakeUnique<FUAFStack>(TUAFStack<ValueType>(StackName));
		return *static_cast<TUAFStack<ValueType>*>(UserDefinedStacks.AddByHash(StackName.GetNameHash(), StackName.GetName(), MoveTemp(Stack)).Get());
	}

	template<typename ValueType>
	inline TUAFStack<ValueType>* FUAFAnimOpEvaluator::FindStack(const FUAFStackName& StackName)
	{
		TUniquePtr<FUAFStack>* Stack = UserDefinedStacks.FindByHash(StackName.GetNameHash(), StackName.GetName());
		checkf(Stack == nullptr || Stack->Get()->GetTypeID() == GetStackTypeID<ValueType>(), TEXT("Type mismatch! This stack is being queried with a different type than it was created with."));
		return static_cast<TUAFStack<ValueType>*>(Stack->Get());
	}

	template<typename ValueType>
	inline const TUAFStack<ValueType>* FUAFAnimOpEvaluator::FindStack(const FUAFStackName& StackName) const
	{
		const TUniquePtr<FUAFStack>* Stack = UserDefinedStacks.FindByHash(StackName.GetNameHash(), StackName.GetName());
		checkf(Stack == nullptr || Stack->Get()->GetTypeID() == GetStackTypeID<ValueType>(), TEXT("Type mismatch! This stack is being queried with a different type than it was created with."));
		return static_cast<const TUAFStack<ValueType>*>(Stack->Get());
	}
}

#undef UE_API
