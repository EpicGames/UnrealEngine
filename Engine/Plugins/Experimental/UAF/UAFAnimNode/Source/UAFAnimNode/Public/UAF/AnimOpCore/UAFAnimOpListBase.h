// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/AdderRef.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFAnimOpListBase.generated.h"

namespace UE::UAF
{
	/**
	 * FUAFAnimOpListBase
	 *
	 * Base type for lists of AnimOps.
	 */
	USTRUCT()
	struct FUAFAnimOpListBase : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFAnimOpListBase)

		// ForceInit constructor for TCppStructOps that creates an invalid AnimOp
		explicit FUAFAnimOpListBase(EForceInit);

		// Returns the list of AnimOps contained within
		virtual void GetAnimOps(TAdderRef<const FUAFAnimOp*> OutAnimOps) const PURE_VIRTUAL(FUAFAnimOpListBase::GetAnimOps,);

	protected:
		FUAFAnimOpListBase();
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline FUAFAnimOpListBase::FUAFAnimOpListBase(EForceInit ForceInit)
		: FUAFAnimOp(ForceInit)
	{
	}

	inline FUAFAnimOpListBase::FUAFAnimOpListBase()
		: FUAFAnimOp(0)	// Lists have 0 inputs since they are just acting as a proxy
	{
	}
}

template<>
struct TStructOpsTypeTraits<UE::UAF::FUAFAnimOpListBase> : TStructOpsTypeTraitsBase2<UE::UAF::FUAFAnimOpListBase>
{
	enum
	{
		WithNoInitConstructor = true,
		WithPureVirtual = true,
	};
};
