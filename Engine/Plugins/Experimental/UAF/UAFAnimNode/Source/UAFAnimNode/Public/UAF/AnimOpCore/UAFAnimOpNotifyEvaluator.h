// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOpEvaluator.h"

#define UE_API UAFANIMNODE_API

struct FAnimNotifyEventReference;

namespace UE::UAF
{
	/*
	 * FUAFAnimOpNotifyEvaluator
	 *
	 * The AnimOp notify evaluator provides the necessary machinery for notify evaluation.
	 */
	class FUAFAnimOpNotifyEvaluator : public FUAFAnimOpEvaluator
	{
	public:
		UE_API FUAFAnimOpNotifyEvaluator();
		UE_API ~FUAFAnimOpNotifyEvaluator();

		// Returns the list of notifies currently being evaluated
		[[nodiscard]] TArray<FAnimNotifyEventReference>& GetNotifies();
		[[nodiscard]] const TArray<FAnimNotifyEventReference>& GetNotifies() const;

	private:
		// The list of notifies being produced
		TArray<FAnimNotifyEventReference> Notifies;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline TArray<FAnimNotifyEventReference>& FUAFAnimOpNotifyEvaluator::GetNotifies()
	{
		return Notifies;
	}

	inline const TArray<FAnimNotifyEventReference>& FUAFAnimOpNotifyEvaluator::GetNotifies() const
	{
		return Notifies;
	}
}

#undef UE_API
