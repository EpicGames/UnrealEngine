// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOpListBase.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"

namespace UE::UAF
{
	/**
	 * TUAFAnimOpTuple
	 *
	 * A tuple of AnimOps.
	 * AnimOps are evaluated in the order specified: left to right.
	 */
	template<typename... AnimOpTypes>
	struct TUAFAnimOpTuple : public FUAFAnimOpListBase
	{
		TUAFAnimOpTuple();

		// Returns the AnimOp with the specified index from the specified order
		template<uint32 Index> UE_REWRITE decltype(auto) Get();
		template<uint32 Index> UE_REWRITE decltype(auto) Get() const;

		// FUAFAnimOp impl
		virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;
		virtual void EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator) override;
		virtual void EvaluateSynchronization(FUAFAnimOpSyncEvaluator& Evaluator) override;

	protected:
		// FUAFAnimOpListBase impl
		virtual void GetAnimOps(TAdderRef<const FUAFAnimOp*> OutAnimOps) const override;

		// A list of AnimOps, evaluated left to right
		TTuple<AnimOpTypes...> AnimOps;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	template<typename... AnimOpTypes>
	inline TUAFAnimOpTuple<AnimOpTypes...>::TUAFAnimOpTuple()
		: FUAFAnimOpListBase()
	{
		InitializeAs<TUAFAnimOpTuple<AnimOpTypes...>>();
	}

	template<typename... AnimOpTypes>
	template<uint32 Index>
	inline decltype(auto) TUAFAnimOpTuple<AnimOpTypes...>::Get()
	{
		return AnimOps.template Get<Index>();
	}

	template<typename... AnimOpTypes>
	template<uint32 Index>
	inline decltype(auto) TUAFAnimOpTuple<AnimOpTypes...>::Get() const
	{
		return AnimOps.template Get<Index>();
	}

	template<typename... AnimOpTypes>
	inline void TUAFAnimOpTuple<AnimOpTypes...>::GetAnimOps(TAdderRef<const FUAFAnimOp*> OutAnimOps) const
	{
		VisitTupleElements([&OutAnimOps](auto& AnimOp) { OutAnimOps.Add(&AnimOp); }, AnimOps);
	}

	template<typename... AnimOpTypes>
	inline void TUAFAnimOpTuple<AnimOpTypes...>::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
	{
		TUAFStack<FPoseValueBundleCoWRef>& EvaluationStack = Evaluator.GetEvaluationStack();

		VisitTupleElements(
			[&Evaluator, &EvaluationStack](auto& AnimOp)
			{
				if (EvaluationStack.Num() < AnimOp.GetNumInputs()) [[unlikely]]
				{
					UE_LOGF(LogAnimation, Warning, "Too few inputs provided, AnimOp will be skipped.");
					return;
				}

				AnimOp.EvaluateValues(Evaluator);
			}, AnimOps);
	}

	template<typename... AnimOpTypes>
	inline void TUAFAnimOpTuple<AnimOpTypes...>::EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator)
	{
		VisitTupleElements(
			[&Evaluator](auto& AnimOp)
			{
				if (!AnimOp.HasEvaluateNotifies()) [[likely]]
				{
					// We skip AnimOps that don't implement EvaluateNotifies since most of them don't
					return;
				}

				AnimOp.EvaluateNotifies(Evaluator);
			}, AnimOps);
	}

	template<typename... AnimOpTypes>
	inline void TUAFAnimOpTuple<AnimOpTypes...>::EvaluateSynchronization(FUAFAnimOpSyncEvaluator& Evaluator)
	{
		VisitTupleElements(
			[&Evaluator](auto& AnimOp)
			{
				AnimOp.EvaluateSynchronization(Evaluator);
			}, AnimOps);
	}
}
