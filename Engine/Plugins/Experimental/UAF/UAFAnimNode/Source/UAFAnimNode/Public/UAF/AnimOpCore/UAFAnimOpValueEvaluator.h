// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOpEvaluator.h"
#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/UAFCoWRef.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"
#include "UAF/ValueRuntime/PoseValueBundle.h"

#define UE_API UAFANIMNODE_API

class UAbstractSkeletonSetBinding;
class USkeleton;

namespace UE::UAF
{
	// A stack of FPoseValueBundle instances used when sampling sequences and blending their results
	// Their container instances will use the MemStack but internally they can use any allocator
	using FPoseValueBundleCoWRef = TUAFCoWRef<FPoseValueBundle, TMemStackAllocator<>>;
	extern UE_API const FUAFStackName VALUE_STACK_NAME;
	UE_UAF_REGISTER_STACK_TYPE(FPoseValueBundleCoWRef);

	/*
	 * FUAFAnimOpValueEvaluationContext
	 *
	 * Encapsulates the state that controls what values AnimOps should produce.
	 */
	class FUAFAnimOpValueEvaluationContext final
	{
	public:
		UE_API FUAFAnimOpValueEvaluationContext();
		UE_API FUAFAnimOpValueEvaluationContext(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName);

		// Returns whether or not this context is valid
		[[nodiscard]] bool IsValid() const;

		// Returns the abstract skeleton set binding asset we are evaluating with (or nullptr if we aren't valid)
		[[nodiscard]] const UAbstractSkeletonSetBinding* GetSetBinding() const;

		// Returns the concrete skeleton we are evaluating with (or nullptr if we aren't valid)
		[[nodiscard]] const USkeleton* GetSkeleton() const;

		// Returns the skeletal mesh we are evaluating with, may be nullptr)
		[[nodiscard]] const USkeletalMesh* GetSkeletalMesh() const;

		// Returns the named set we are currently evaluating with (or nullptr if we aren't valid)
		[[nodiscard]] const FAttributeNamedSetPtr& GetNamedSet() const;

		// Sets the named set
		// Returns true if we succeeded and the set exists in the current binding, false otherwise
		[[nodiscard]] UE_API bool SetNamedSet(FName SetName);

		// Returns the default attribute values for the current named set (or nullptr if we aren't valid)
		[[nodiscard]] const FPoseValueBundlePtr& GetDefaultAttributeValues() const;

	private:
		// The set binding we are evaluating with
		const UAbstractSkeletonSetBinding* SetBinding = nullptr;

		// The target skeletal mesh with are evaluating with, may be nullptr
		const USkeletalMesh* SkeletalMesh = nullptr;

		// The target skeleton we are evaluating with
		const USkeleton* Skeleton = nullptr;

		// Binding data owned by the set binding
		FAttributeBindingDataPtr BindingData;

		// Named set instance to produce an output for
		FAttributeNamedSetPtr NamedSet;

		// Default values as specified by the current set binding and named set
		FPoseValueBundlePtr DefaultAttributeValues;
	};

	/*
	 * FUAFAnimOpValueEvaluator
	 *
	 * The AnimOp value evaluator provides the necessary machinery for value evaluation.
	 */
	class FUAFAnimOpValueEvaluator : public FUAFAnimOpEvaluator
	{
	public:
		UE_API FUAFAnimOpValueEvaluator(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName, int32 CurrentLOD);

		// Returns the evaluation stack for value bundles
		[[nodiscard]] TUAFStack<FPoseValueBundleCoWRef>& GetEvaluationStack();
		[[nodiscard]] const TUAFStack<FPoseValueBundleCoWRef>& GetEvaluationStack() const;

		// Returns the evaluation context that controls what values we produce
		[[nodiscard]] UE_API const FUAFAnimOpValueEvaluationContext& GetActiveEvaluationContext() const;

		// Returns the named set we are currently evaluating with
		[[nodiscard]] UE_API const FAttributeNamedSetPtr& GetActiveNamedSet() const;

		// Pushes a new binding and named set to evaluate with
		// Returns true if we succeeded and the set exists in the specified binding, false otherwise
		[[nodiscard]] UE_API bool PushEvaluationContext(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh, FName SetName);

		// Pushes a new named set to evaluate with
		// Returns true if we succeeded and the set exists in the currently active binding, false otherwise
		[[nodiscard]] UE_API bool PushEvaluationContext(FName SetName);

		// Pops the current active named set
		// Returns whether or not we succeeded (we need to retain at least one named set)
		UE_API bool PopEvaluationContext();

		// Returns the current LOD
		[[nodiscard]] int32 GetCurrentLOD() const;

		// Returns the value transformer map
		[[nodiscard]] const FValueTransformerMapPtr& GetTransformerMap() const;

	private:
		// The main evaluation stack for values
		TUAFStack<FPoseValueBundleCoWRef> EvaluationStack;

		// The internal context that controls what values we are producing
		TArray<FUAFAnimOpValueEvaluationContext> EvaluationContextStack;

		// Shared map of value transformers
		FValueTransformerMapPtr ValueTransformerMap;

		// Current LOD we are evaluating at
		int32 CurrentLOD = 0;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline const UAbstractSkeletonSetBinding* FUAFAnimOpValueEvaluationContext::GetSetBinding() const
	{
		return SetBinding;
	}

	inline const USkeleton* FUAFAnimOpValueEvaluationContext::GetSkeleton() const
	{
		return Skeleton;
	}

	inline const USkeletalMesh* FUAFAnimOpValueEvaluationContext::GetSkeletalMesh() const
	{
		return SkeletalMesh;
	}

	inline const FAttributeNamedSetPtr& FUAFAnimOpValueEvaluationContext::GetNamedSet() const
	{
		return NamedSet;
	}

	inline const FPoseValueBundlePtr& FUAFAnimOpValueEvaluationContext::GetDefaultAttributeValues() const
	{
		return DefaultAttributeValues;
	}

	inline bool FUAFAnimOpValueEvaluationContext::IsValid() const
	{
		return SetBinding != nullptr;
	}

	inline TUAFStack<FPoseValueBundleCoWRef>& FUAFAnimOpValueEvaluator::GetEvaluationStack()
	{
		return EvaluationStack;
	}

	inline const TUAFStack<FPoseValueBundleCoWRef>& FUAFAnimOpValueEvaluator::GetEvaluationStack() const
	{
		return EvaluationStack;
	}

	inline int32 FUAFAnimOpValueEvaluator::GetCurrentLOD() const
	{
		return CurrentLOD;
	}

	inline const FValueTransformerMapPtr& FUAFAnimOpValueEvaluator::GetTransformerMap() const
	{
		return ValueTransformerMap;
	}
}

#undef UE_API
