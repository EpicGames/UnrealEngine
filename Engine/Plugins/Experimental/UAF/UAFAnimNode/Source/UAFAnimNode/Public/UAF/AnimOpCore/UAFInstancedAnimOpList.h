// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"

#include "UAFInstancedAnimOpList.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	struct FUAFAnimOp;

	USTRUCT()
	struct FUAFInstancedAnimOpList
	{
		GENERATED_BODY()

		FUAFInstancedAnimOpList() = default;
		UE_API FUAFInstancedAnimOpList(const TArray<FUAFAnimOp*>& AnimOps);

		UPROPERTY(EditAnywhere, Category = "AnimOps", meta = (ExpandByDefault))
		TArray<FInstancedStruct> AnimOps;
	};
}

#undef UE_API
