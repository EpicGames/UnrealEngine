// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"
#include "UAF/BlendMask/UAFBlendMask.h"

#include "UAFApplyAdditiveAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	class FValueBundleStack;
	class FPoseValueBundle;

	USTRUCT()
	struct FUAFApplyAdditiveAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFApplyAdditiveAnimOp)
		
	public:
		FUAFApplyAdditiveAnimOp();
		
		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;
		
		UE_API void SetAdditiveWeight(float InWeight);
		UE_API void SetBlendMask(const TObjectPtr<UUAFBlendMask> InBlendMask);
		
	private:
		void BuildPerValueWeights(const FUAFAnimOpValueEvaluator& Evaluator, const FPoseValueBundle& BaseInput, FValueBundleStack& OutPerValueWeights);
		
	private:
		// Optional blend mask for per-bone application of the additive
		UPROPERTY(Transient)
		TObjectPtr<UUAFBlendMask> OptionalBlendMask = nullptr;
		
		// The uniform weight to apply the additive with 
		float AdditiveWeight = 1.0f;
	};
}

#undef UE_API
