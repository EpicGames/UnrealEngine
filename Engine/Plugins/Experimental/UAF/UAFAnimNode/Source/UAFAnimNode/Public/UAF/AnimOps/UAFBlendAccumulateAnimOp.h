// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFBlendAccumulateAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFBlendAccumulateAnimOp
	 *
	 * This pops the top two evaluation stack results (A and B, let B be at the top).
	 * B is the input that we accumulate on top of; while A is the input we scale.
	 * The result is pushed back onto the stack.
	 * Top = B + (A * ScaleFactor)
	 * Top = Top + (Top-1 * ScaleFactor)
	 */
	USTRUCT()
	struct FUAFBlendAccumulateAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFBlendAccumulateAnimOp)

		UE_API FUAFBlendAccumulateAnimOp();

		// Returns the scale factor to apply to the top-1 result of the evaluation stack
		float GetScaleFactor() const;

		// Sets the scale factor to apply to the top-1 result of the evaluation stack
		void SetScaleFactor(float ScaleFactor);

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	private:
		// The scale factor to apply to the keyframe
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float ScaleFactor = 0.0f;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline float FUAFBlendAccumulateAnimOp::GetScaleFactor() const
	{
		return ScaleFactor;
	}

	inline void FUAFBlendAccumulateAnimOp::SetScaleFactor(float InScaleFactor)
	{
		ScaleFactor = InScaleFactor;
	}
}

#undef UE_API
