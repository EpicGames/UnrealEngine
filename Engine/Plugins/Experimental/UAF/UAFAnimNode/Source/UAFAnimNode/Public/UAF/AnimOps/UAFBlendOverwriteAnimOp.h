// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFBlendOverwriteAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFBlendOverwriteAnimOp
	 *
	 * This pops the top result from the evaluation stack, scales it by a weight factor, then pushes back the result.
	 */
	USTRUCT()
	struct FUAFBlendOverwriteAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFBlendOverwriteAnimOp)

		UE_API FUAFBlendOverwriteAnimOp();

		// Returns the scale factor to apply to the top result of the evaluation stack
		float GetScaleFactor() const;

		// Sets the scale factor to apply to the top result of the evaluation stack
		void SetScaleFactor(float ScaleFactor);

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	private:
		// The scale factor to apply to the top result of the evaluations tack
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float ScaleFactor = 1.0f;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline float FUAFBlendOverwriteAnimOp::GetScaleFactor() const
	{
		return ScaleFactor;
	}

	inline void FUAFBlendOverwriteAnimOp::SetScaleFactor(float InScaleFactor)
	{
		ScaleFactor = InScaleFactor;
	}
}

#undef UE_API
