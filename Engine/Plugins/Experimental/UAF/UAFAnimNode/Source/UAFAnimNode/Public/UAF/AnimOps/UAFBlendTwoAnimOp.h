// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFBlendTwoAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	class FPoseValueBundle;
}

namespace UE::UAF
{
	/**
	 * FUAFBlendTwoAnimOp
	 *
	 * A simple AnimOp that blends two inputs.
	 * 
	 * This pops the top two inputs from the evaluation stack, it interpolates them, and pushes
	 * back the result onto the evaluation stack.
	 * Let B be the input keyframe at the top of the stack and A be the second from the top.
	 * Then we have:
	 *     Result = Interpolate(A, B, Alpha)
	 * An Alpha of 0.0 returns A while 1.0 returns B (top)
	 */
	USTRUCT()
	struct FUAFBlendTwoAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFBlendTwoAnimOp)

		UE_API FUAFBlendTwoAnimOp();

		UE_API void SetInterpolationAlpha(float InterpolationAlpha);
		UE_API void SetInterpolationCurve(const FName& AlphaSourceCurveName, const int8 AlphaCurveInputIndex, TFunction<float(float)> InputScaleBiasClampFn);

		UE_API void SetBlendOption(EAlphaBlendOption InBlendOption);
		
		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	protected:
		// Compute the current alpha value for curves or return the interpolation alpha otherwise
		[[nodiscard]] UE_API float GetInterpolationAlpha(const FPoseValueBundle& InputA, const FPoseValueBundle& InputB) const;

		// The desired interpolation alpha between the two input keyframes
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float DesiredInterpolationAlpha = 0.0f;

		// The actual interpolation alpha between the two input keyframes
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float ActualInterpolationAlpha = 0.0f;

		// The curve to evaluate and extract the interpolation alpha between the two input keyframes
		UPROPERTY(VisibleAnywhere, Category = Properties)
		FName AlphaSourceCurveName = NAME_None;

		// Which input to take our curve from (0 for A or 1 for B)
		UPROPERTY(VisibleAnywhere, Category = Properties)
		int8 AlphaCurveInputIndex = INDEX_NONE;

		UPROPERTY(VisibleAnywhere, Category = Properties)
		EAlphaBlendOption BlendOption = UE::Anim::DefaultBlendOption;
		
		TFunction<float(float)> InputScaleBiasClampFn;
	};
}

#undef UE_API
