// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AlphaBlend.h"
#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFBlendTwoPerValueAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

class UUAFBlendProfile;

namespace UE::UAF
{
	USTRUCT()
	struct FUAFBlendTwoPerValueAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFBlendTwoPerValueAnimOp)

		UE_API FUAFBlendTwoPerValueAnimOp();
		UE_API FUAFBlendTwoPerValueAnimOp(TNonNullPtr<UUAFBlendProfile> BlendProfile, EAlphaBlendOption BlendOption);

		UE_API void SetLinearWeightB(float InterpolationAlpha);

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	protected:
		// The interpolation alpha between the two input keyframes
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float LinearGlobalWeightB = 0.0f;

		UPROPERTY(VisibleAnywhere, Category = Properties)
		TObjectPtr<UUAFBlendProfile> BlendProfile = nullptr;

		UPROPERTY(VisibleAnywhere, Category = Properties)
		EAlphaBlendOption BlendOption = UE::Anim::DefaultBlendOption;
	};
}

#undef UE_API