// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequenceBase.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFDecompressAnimSequenceAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFDecompressAnimSequenceAnimOp
	 *
	 * Decompresses the data from an animation sequence.
	 * Produces: values, notifies, and sync contribution.
	 */
	USTRUCT()
	struct FUAFDecompressAnimSequenceAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFDecompressAnimSequenceAnimOp)

		UE_API FUAFDecompressAnimSequenceAnimOp();

		// Initializes the AnimOp
		UE_API void Initialize(TObjectPtr<const UAnimSequence> AnimSequence, float StartTime, bool bIsLooping, bool bInterpolate, bool bExtractTrajectory);

		// Advances time by the specified delta
		UE_API ETypeAdvanceAnim AdvanceTime(float DeltaTime);

		// Returns the current animation sequence being decompressed
		const UAnimSequence* GetAnimSequence() const;

		// Returns the current time being sampled
		float GetCurrentTime() const;

		// Returns the previous time being sampled
		float GetPreviousTime() const;

		// Returns the animation sequence duration
		float GetDuration() const;

		// Returns how time was advanced in the last call to AdvanceTime(..)
		ETypeAdvanceAnim GetTimeAdvanceResult() const;

		// Returns whether or not this AnimOp is looping
		bool IsLooping() const;

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;
		UE_API virtual void EvaluateNotifies(FUAFAnimOpNotifyEvaluator& Evaluator) override;

	private:
		// Anim Sequence to grab the keyframe from
		UPROPERTY(VisibleAnywhere, Category = Properties)
		TObjectPtr<const UAnimSequence> AnimSequence;

		/** Delta time range required for root motion extraction **/
		FDeltaTimeRecord DeltaTimeRecord;

		// The current playback position within the animation sequence.
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float CurrentTime = 0.0f;

		// The previous playback position within the animation sequence.
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float PreviousTime = 0.0f;

		// The animation sequence duration
		UPROPERTY(VisibleAnywhere, Category = Properties)
		float Duration = 0.0f;

		// How time advanced in the last call to AdvanceTime(..)
		UPROPERTY(VisibleAnywhere, Category = Properties)
		TEnumAsByte<ETypeAdvanceAnim> TimeAdvanceResult = ETAA_Default;

		// Whether to interpolate or step the animation sequence.
		// Only used when the sample time is used.
		UPROPERTY(VisibleAnywhere, Category = Properties)
		bool bInterpolate = false;

		// Whether to extract trajectory or not
		UPROPERTY(VisibleAnywhere, Category = Properties)
		bool bExtractTrajectory = false;

		UPROPERTY(VisibleAnywhere, Category = Properties)
		bool bIsLooping = false;
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline const UAnimSequence* FUAFDecompressAnimSequenceAnimOp::GetAnimSequence() const
	{
		return AnimSequence;
	}

	inline float FUAFDecompressAnimSequenceAnimOp::GetCurrentTime() const
	{
		return CurrentTime;
	}

	inline float FUAFDecompressAnimSequenceAnimOp::GetPreviousTime() const
	{
		return PreviousTime;
	}

	inline float FUAFDecompressAnimSequenceAnimOp::GetDuration() const
	{
		return Duration;
	}

	inline ETypeAdvanceAnim FUAFDecompressAnimSequenceAnimOp::GetTimeAdvanceResult() const
	{
		return TimeAdvanceResult;
	}

	inline bool FUAFDecompressAnimSequenceAnimOp::IsLooping() const
	{
		return bIsLooping;
	}
}

#undef UE_API
