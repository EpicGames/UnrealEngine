// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFMakeDynamicAdditiveAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	class FUAFAnimOpValueEvaluator;

	// Anim Op to generate an additive animation from Base and Source poses.
	// The output is (Source - Base) from the stack.
	USTRUCT()
	struct FUAFMakeDynamicAdditiveAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFMakeDynamicAdditiveAnimOp)

	public:
		FUAFMakeDynamicAdditiveAnimOp();

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

		UE_API void SetMeshSpaceAdditive(bool bInMeshSpaceAdditive);

	private:

		// Whether to compute the additive delta in mesh space (rotation only).
		UPROPERTY(VisibleAnywhere, Category = Properties)
		bool bMeshSpaceAdditive = false;
	};
}

#undef UE_API
