// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFNullAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFNullAnimOp
	 *
	 * A dummy AnimOp that produces empty outputs.
	 * Leaf anim nodes that cannot produce a meaningful result should produce a Null AnimOp.
	 * This way, a parent AnimOp can reason meaningfully about the missing input.
	 * 
	 * When possible, use the globally static instance through FUAFNullAnimOp::Get().
	 */
	USTRUCT()
	struct FUAFNullAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFNullAnimOp)

		UE_API FUAFNullAnimOp();

		// Returns a globally static instance of a Null AnimOp
		[[nodiscard]] UE_API static FUAFNullAnimOp* Get();

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;
	};
}

#undef UE_API
