// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"
#include "UAFOffsetRootBoneAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

UENUM(BlueprintType)
enum class EUAFOffsetRootBoneNodeMode : uint8
{
	// Accumulate the mesh component's movement into the offset.
	// In this mode, if the mesh component moves
	// the offset will counter the motion, and the root will stay in place
	Accumulate,
	// Continuously interpolate the offset out
	// In this mode, if the mesh component moves
	// The root will stay behind, but will attempt to catch up
	Interpolate,
	// Release the offset and stop accumulating the mesh component's movement delta.
	// In this mode we will "blend out" the offset
	Release,
};

namespace UE::UAF
{

USTRUCT()
struct FUAFOffsetRootBoneAnimOp : public FUAFAnimOp
{
	GENERATED_BODY()
	UAF_DECLARE_ANIMOP(FUAFOffsetRootBoneAnimOp)

	FUAFOffsetRootBoneAnimOp();

	// FUAFAnimOp impl
	UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	// Per-frame resolved values (set by Node's PostUpdate)

	UPROPERTY()
	float DeltaTime = 0.f;

	UPROPERTY()
	float Alpha = 1.0f;

	UPROPERTY()
	FTransform MeshComponentTransformWorld = FTransform::Identity;

	UPROPERTY()
	EUAFOffsetRootBoneNodeMode TranslationMode = EUAFOffsetRootBoneNodeMode::Interpolate;

	UPROPERTY()
	EUAFOffsetRootBoneNodeMode RotationMode = EUAFOffsetRootBoneNodeMode::Interpolate;

	UPROPERTY()
	float TranslationSmoothingTime = 0.1f;

	UPROPERTY()
	float RotationSmoothingTime = 0.2f;

	UPROPERTY()
	float MaxTranslationError = -1.0f;

	UPROPERTY()
	float MaxRotationErrorDegrees = -1.0f;

	UPROPERTY()
	bool bOnGround = true;

	UPROPERTY()
	FVector AnimatedGroundNormal = FVector(0, 0, 1);

	// Constant values (set once in Node constructor)

	UPROPERTY()
	bool bClampToTranslationVelocity = false;

	UPROPERTY()
	bool bClampToRotationVelocity = false;

	UPROPERTY()
	float TranslationSpeedRatio = 0.5f;

	UPROPERTY()
	float RotationSpeedRatio = 0.5f;

	UPROPERTY()
	float TeleportDistanceThreshold = 300.0f;

	UPROPERTY()
	bool bResetOnTeleport = false;

	// Persistent state (survives across frames)

	bool bIsFirstUpdate = true;
	FTransform LastMeshComponentTransformWorld = FTransform::Identity;
	FVector SimulatedTranslation = FVector::ZeroVector;
	FQuat SimulatedRotation = FQuat::Identity;

	UObject* HostObject = nullptr;
};

} // namespace UE::UAF

#undef UE_API
