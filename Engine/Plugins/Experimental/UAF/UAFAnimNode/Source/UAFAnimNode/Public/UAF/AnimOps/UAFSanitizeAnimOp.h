// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFSanitizeAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
	/**
	 * FUAFSanitizeAnimOp
	 *
	 * Sanitizes an input value.
	 * For pose values, rotations are normalized.
	 * For float values, we validate that values are not NaN/Inf.
	 */
	USTRUCT()
	struct FUAFSanitizeAnimOp : public FUAFAnimOp
	{
		GENERATED_BODY()
		UAF_DECLARE_ANIMOP(FUAFSanitizeAnimOp)

		UE_API FUAFSanitizeAnimOp();

		// FUAFAnimOp impl
		UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;
	};
}

#undef UE_API
