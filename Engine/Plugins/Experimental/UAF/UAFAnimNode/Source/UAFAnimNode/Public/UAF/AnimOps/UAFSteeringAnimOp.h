// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"

#include "UAFSteeringAnimOp.generated.h"

#define UE_API UAFANIMNODE_API

namespace UE::UAF
{
struct IUAFAnimNodeTimeline;
struct IUAFRootMotionProvider;
}

namespace UE::UAF
{
USTRUCT()
struct FUAFSteeringAnimOp : public FUAFAnimOp
{
	GENERATED_BODY()
	UAF_DECLARE_ANIMOP(FUAFSteeringAnimOp)

	FUAFSteeringAnimOp();

	// FUAFAnimOp impl
	UE_API virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	UPROPERTY()
	float DeltaTime = 0.f;

	UPROPERTY()
	float Alpha = 1.0f;

	UPROPERTY()
	FVector AngularVelocity = FVector::ZeroVector;

	UPROPERTY()
	FQuat TargetOrientation = FQuat::Identity;

	UPROPERTY()
	FTransform RootBoneTransform = FTransform::Identity;

	UPROPERTY()
	float ProceduralTargetTime = 0.2f;

	UPROPERTY()
	float AnimatedTargetTime = 0.f;

	UPROPERTY()
	float DisableSteeringBelowSpeed = 0.f;

	UPROPERTY()
	float RootMotionAngleThresholdDegrees = 1.0f;

	// below this movement speed (based on the root motion in the animation) disable steering coming from the additive spring based correction
	UPROPERTY()
	float DisableAdditiveBelowSpeed = -1.0f;

	// Will clamp the scaling ratio applied to above this threashold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY()
	float MinScaleRatio = 0.5f;

	// Will clamp the scaling ratio applied to below this threashold. Any error remaining will be compensated for linearly (using ProceduralTargetTime)
	UPROPERTY()
	float MaxScaleRatio = 1.5f;

	UObject* HostObject = nullptr;
	
	// These interfaces are needed if AnimatedTargetTime is greater than 0 (i.e. if we want to take into account predicted root motion)
	IUAFRootMotionProvider* RootMotionProvider = nullptr;
	IUAFAnimNodeTimeline* Timeline = nullptr;
};
}

#undef UE_API