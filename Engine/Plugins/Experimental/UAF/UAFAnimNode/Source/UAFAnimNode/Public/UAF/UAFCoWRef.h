// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/ValueRuntime/IndirectAllocator.h"

namespace UE::UAF
{
	/**
	 * TUAFCoWRef
	 *
	 * A reference to a value type that performs Copy-on-Write.
	 * A reference is never invalid, it must always be constructed from valid data.
	 * 
	 * A CoW Reference tracks 3 things:
	 *   - A pointer to an allocated value
	 *   - Whether or not it owns the allocated memory (if we own it, the reference is unique)
	 *   - Whether or not the reference is immutable
	 * 
	 * For performance reasons, this tracking is performed using a single pointer where the bottom two bits
	 * are used to store our metadata. We thus require an alignment of 4 bytes for all allocations.
	 * 
	 * Mutability is enforced at runtime not through the type system (e.g. with 'const').
	 * The specified allocator will be used when a copy is made and memory is owned by the reference.
	 * 
	 * Copying or assigning a reference to another always performs a copy where the new reference
	 * owns the resulting memory. This occurs regardless whether or not the source/target are mutable/immutable.
	 * 
	 * It is always safe to query for a const reference using TUAFCoWRef::Get() regardless of whether or
	 * not the underlying reference is immutable. However, care must be taken when using TUAFCoWRef::GetMutable().
	 * It is the responsibility of the caller to ensure that the reference is mutable before attempting to
	 * use it as such. If the reference is not mutable, TUAFCoWRef::ForceMutable() can be used to convert it.
	 * 
	 * Two allocators are supported: FDefaultAllocator and TMemStackAllocator.
	 */
	template<class ValueType, class AllocatorType = FDefaultAllocator>
	class TUAFCoWRef
	{
	public:
		// Constructs a new mutable reference from the specified value by performing a move of the input.
		// The reference will be unique as its memory will be owned by us.
		[[nodiscard]] static TUAFCoWRef<ValueType, AllocatorType> MakeFrom(ValueType&& Value);

		// Constructs a new mutable reference from the specified value pointer.
		// The reference will not be unique as its memory will not be owned by us.
		[[nodiscard]] static TUAFCoWRef<ValueType, AllocatorType> MakeMutable(ValueType* ValuePtr);

		// Constructs a new immutable reference from the specified value pointer.
		// The reference will not be unique as its memory will not be owned by us.
		[[nodiscard]] static TUAFCoWRef<ValueType, AllocatorType> MakeImmutable(const ValueType* ValuePtr);

		// Constructs a new mutable reference by performing a copy of the input reference.
		// The reference will be unique as its memory will be owned by us.
		TUAFCoWRef(const TUAFCoWRef<ValueType, AllocatorType>& Other);

		// Constructs a new mutable reference by performing a copy of the input reference.
		// The reference will be unique as its memory will be owned by us.
		template<class OtherAllocatorType>
		TUAFCoWRef(const TUAFCoWRef<ValueType, OtherAllocatorType>& Other);

		// Move a reference into another
		TUAFCoWRef(TUAFCoWRef<ValueType, AllocatorType>&& Other);

		// Destroys the current reference
		// If it owned the memory within, it is freed
		~TUAFCoWRef();
		
		// Assign a new mutable reference by performing a copy of the input reference.
		// The reference will be unique as its memory will be owned by us.
		TUAFCoWRef<ValueType, AllocatorType>& operator=(const TUAFCoWRef<ValueType, AllocatorType>& Other);

		// Assign a new mutable reference by performing a copy of the input reference.
		// The reference will be unique as its memory will be owned by us.
		template<class OtherAllocatorType>
		TUAFCoWRef<ValueType, AllocatorType>& operator=(const TUAFCoWRef<ValueType, OtherAllocatorType>& Other);

		// Move assign a reference into another
		TUAFCoWRef<ValueType, AllocatorType>& operator=(TUAFCoWRef<ValueType, AllocatorType>&& Other);

		// Move assign a value into a reference by performing a move of the input.
		// The reference will be unique as its memory will be owned by us.
		TUAFCoWRef<ValueType, AllocatorType>& operator=(ValueType&& Value);

		// Returns an immutable reference to the value contained within
		[[nodiscard]] const ValueType& Get() const;

		// Returns a mutable reference to the value contained within
		[[nodiscard]] ValueType& GetMutable();

		// Ensures that this reference is mutable
		// If the current reference was immutable and we do not own the memory, this performs a copy (Copy-on-Write)
		// Note that any existing const references will remain valid since the original owner is keeping it alive
		void ForceMutable();

		// Returns whether or not this reference is mutable
		[[nodiscard]] bool IsMutable() const;

		// Returns whether or not this reference is unique
		// A reference is unique if we own its memory
		[[nodiscard]] bool IsUnique() const;

	private:
		static constexpr uintptr_t VALUE_PTR_MASK = static_cast<uintptr_t>(~0x0llu & ~0x03llu);
		static constexpr uintptr_t FLAGS_MASK = 0x03llu;
		static constexpr uintptr_t IS_MUTABLE_FLAG_BIT = 0x02llu;
		static constexpr uintptr_t OWNS_MEMORY_FLAG_BIT = 0x01llu;

		// Disable construction of empty references
		TUAFCoWRef() = default;

		ValueType* GetValuePtr();
		const ValueType* GetValuePtr() const;

		// Stores a pointer to a value and 2 flags in the LSBs
		uintptr_t ValuePtrAndFlags = 0;
	};

	// Returns a mutable reference from a list of potential candidates.
	// If candidates are mutable, the first one will be returned.
	// Otherwise, the first unique immutable reference is returned.
	// If none of the candidates are mutable or unique, we perform a Copy-on-Write with the last immutable reference.
	template<class ValueType, class AllocatorType>
	TUAFCoWRef<ValueType, AllocatorType>& FindMutableCoWRef(TUAFCoWRef<ValueType, AllocatorType>& Arg0, TUAFCoWRef<ValueType, AllocatorType>& Arg1);

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType> TUAFCoWRef<ValueType, AllocatorType>::MakeFrom(ValueType&& Value)
	{
		TUAFCoWRef<ValueType, AllocatorType> Result;

		uint8* CopyPtr = FAllocatorTypeTrait<AllocatorType>::Realloc(nullptr, 0, sizeof(ValueType));
		new(CopyPtr) ValueType(MoveTemp(Value));

		Result.ValuePtrAndFlags = reinterpret_cast<uintptr_t>(CopyPtr);
		checkf((Result.ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
		Result.ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
		Result.ValuePtrAndFlags |= OWNS_MEMORY_FLAG_BIT;

		return Result;
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType> TUAFCoWRef<ValueType, AllocatorType>::MakeMutable(ValueType* ValuePtr)
	{
		checkf(ValuePtr != nullptr, TEXT("Cannot create a CoW reference from a null pointer."));

		TUAFCoWRef<ValueType, AllocatorType> Result;

		Result.ValuePtrAndFlags = reinterpret_cast<uintptr_t>(ValuePtr);
		checkf((Result.ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
		Result.ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;

		return Result;
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType> TUAFCoWRef<ValueType, AllocatorType>::MakeImmutable(const ValueType* ValuePtr)
	{
		checkf(ValuePtr != nullptr, TEXT("Cannot create a CoW reference from a null pointer."));

		TUAFCoWRef<ValueType, AllocatorType> Result;

		Result.ValuePtrAndFlags = reinterpret_cast<uintptr_t>(ValuePtr);
		checkf((Result.ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));

		return Result;
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>::TUAFCoWRef(const TUAFCoWRef<ValueType, AllocatorType>& Other)
	{
		uint8* CopyPtr = FAllocatorTypeTrait<AllocatorType>::Realloc(nullptr, 0, sizeof(ValueType));
		new(CopyPtr) ValueType(*Other.GetValuePtr());

		ValuePtrAndFlags = reinterpret_cast<uintptr_t>(CopyPtr);
		checkf((ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
		ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
		ValuePtrAndFlags |= OWNS_MEMORY_FLAG_BIT;
	}

	template<class ValueType, class AllocatorType>
	template<class OtherAllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>::TUAFCoWRef(const TUAFCoWRef<ValueType, OtherAllocatorType>& Other)
	{
		uint8* CopyPtr = FAllocatorTypeTrait<AllocatorType>::Realloc(nullptr, 0, sizeof(ValueType));
		new(CopyPtr) ValueType(*Other.GetValuePtr());

		ValuePtrAndFlags = reinterpret_cast<uintptr_t>(CopyPtr);
		checkf((ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
		ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
		ValuePtrAndFlags |= OWNS_MEMORY_FLAG_BIT;
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>::TUAFCoWRef(TUAFCoWRef<ValueType, AllocatorType>&& Other)
		: ValuePtrAndFlags(Other.ValuePtrAndFlags)
	{
		Other.ValuePtrAndFlags = 0;
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>::~TUAFCoWRef()
	{
		if ((ValuePtrAndFlags & OWNS_MEMORY_FLAG_BIT) != 0)
		{
			ValueType* ValuePtr = GetValuePtr();

			ValuePtr->~ValueType();
			FAllocatorTypeTrait<AllocatorType>::Realloc(reinterpret_cast<uint8*>(ValuePtr), sizeof(ValueType), 0);
		}
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>& TUAFCoWRef<ValueType, AllocatorType>::operator=(const TUAFCoWRef<ValueType, AllocatorType>& Other)
	{
		if (this != &Other)
		{
			if ((ValuePtrAndFlags & OWNS_MEMORY_FLAG_BIT) != 0)
			{
				ValueType* ValuePtr = GetValuePtr();

				ValuePtr->~ValueType();
				FAllocatorTypeTrait<AllocatorType>::Realloc(reinterpret_cast<uint8*>(ValuePtr), sizeof(ValueType), 0);
			}

			uint8* CopyPtr = FAllocatorTypeTrait<AllocatorType>::Realloc(nullptr, 0, sizeof(ValueType));
			new(CopyPtr) ValueType(*Other.GetValuePtr());

			ValuePtrAndFlags = reinterpret_cast<uintptr_t>(CopyPtr);
			checkf((ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
			ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
			ValuePtrAndFlags |= OWNS_MEMORY_FLAG_BIT;
		}

		return *this;
	}

	template<class ValueType, class AllocatorType>
	template<class OtherAllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>& TUAFCoWRef<ValueType, AllocatorType>::operator=(const TUAFCoWRef<ValueType, OtherAllocatorType>& Other)
	{
		if ((ValuePtrAndFlags & OWNS_MEMORY_FLAG_BIT) != 0)
		{
			ValueType* ValuePtr = GetValuePtr();

			ValuePtr->~ValueType();
			FAllocatorTypeTrait<AllocatorType>::Realloc(reinterpret_cast<uint8*>(ValuePtr), sizeof(ValueType), 0);
		}

		uint8* CopyPtr = FAllocatorTypeTrait<AllocatorType>::Realloc(nullptr, 0, sizeof(ValueType));
		new(CopyPtr) ValueType(*Other.GetValuePtr());

		ValuePtrAndFlags = reinterpret_cast<uintptr_t>(CopyPtr);
		checkf((ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
		ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
		ValuePtrAndFlags |= OWNS_MEMORY_FLAG_BIT;

		return *this;
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>& TUAFCoWRef<ValueType, AllocatorType>::operator=(TUAFCoWRef<ValueType, AllocatorType>&& Other)
	{
		if (this != &Other)
		{
			if ((ValuePtrAndFlags & OWNS_MEMORY_FLAG_BIT) != 0)
			{
				ValueType* ValuePtr = GetValuePtr();

				ValuePtr->~ValueType();
				FAllocatorTypeTrait<AllocatorType>::Realloc(reinterpret_cast<uint8*>(ValuePtr), sizeof(ValueType), 0);
			}

			ValuePtrAndFlags = Other.ValuePtrAndFlags;
			Other.ValuePtrAndFlags = 0;
		}

		return *this;
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>& TUAFCoWRef<ValueType, AllocatorType>::operator=(ValueType&& Value)
	{
		if ((ValuePtrAndFlags & OWNS_MEMORY_FLAG_BIT) != 0)
		{
			ValueType* ValuePtr = GetValuePtr();

			ValuePtr->~ValueType();
			FAllocatorTypeTrait<AllocatorType>::Realloc(reinterpret_cast<uint8*>(ValuePtr), sizeof(ValueType), 0);
		}

		uint8* CopyPtr = FAllocatorTypeTrait<AllocatorType>::Realloc(nullptr, 0, sizeof(ValueType));
		new(CopyPtr) ValueType(MoveTemp(Value));

		ValuePtrAndFlags = reinterpret_cast<uintptr_t>(CopyPtr);
		checkf((ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
		ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
		ValuePtrAndFlags |= OWNS_MEMORY_FLAG_BIT;

		return *this;
	}

	template<class ValueType, class AllocatorType>
	inline const ValueType& TUAFCoWRef<ValueType, AllocatorType>::Get() const
	{
		return *GetValuePtr();
	}

	template<class ValueType, class AllocatorType>
	inline ValueType& TUAFCoWRef<ValueType, AllocatorType>::GetMutable()
	{
		checkf((ValuePtrAndFlags & IS_MUTABLE_FLAG_BIT) != 0, TEXT("Attempting to get a mutable reference from an immutable CoW Ref."));
		return *GetValuePtr();
	}

	template<class ValueType, class AllocatorType>
	inline void TUAFCoWRef<ValueType, AllocatorType>::ForceMutable()
	{
		if ((ValuePtrAndFlags & IS_MUTABLE_FLAG_BIT) != 0) [[likely]]
		{
			// We are mutable, all good
			return;
		}
		else
		{
			// We are immutable
			if ((ValuePtrAndFlags & OWNS_MEMORY_FLAG_BIT) != 0)
			{
				// We own the memory, mark is as mutable
				ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
				return;
			}

			// We do not own the memory, make a copy (Copy-on-Write)
			uint8* CopyPtr = FAllocatorTypeTrait<AllocatorType>::Realloc(nullptr, 0, sizeof(ValueType));
			new(CopyPtr) ValueType(*GetValuePtr());

			ValuePtrAndFlags = reinterpret_cast<uintptr_t>(CopyPtr);
			checkf((ValuePtrAndFlags & FLAGS_MASK) == 0, TEXT("Pointer must be aligned to 4 bytes to support CoW."));
			ValuePtrAndFlags |= IS_MUTABLE_FLAG_BIT;
			ValuePtrAndFlags |= OWNS_MEMORY_FLAG_BIT;
		}
	}

	template<class ValueType, class AllocatorType>
	inline bool TUAFCoWRef<ValueType, AllocatorType>::IsMutable() const
	{
		return (ValuePtrAndFlags & IS_MUTABLE_FLAG_BIT) != 0;
	}

	template<class ValueType, class AllocatorType>
	inline bool TUAFCoWRef<ValueType, AllocatorType>::IsUnique() const
	{
		return (ValuePtrAndFlags & OWNS_MEMORY_FLAG_BIT) != 0;
	}

	template<class ValueType, class AllocatorType>
	inline ValueType* TUAFCoWRef<ValueType, AllocatorType>::GetValuePtr()
	{
		return reinterpret_cast<ValueType*>(ValuePtrAndFlags & VALUE_PTR_MASK);
	}

	template<class ValueType, class AllocatorType>
	inline const ValueType* TUAFCoWRef<ValueType, AllocatorType>::GetValuePtr() const
	{
		return reinterpret_cast<const ValueType*>(ValuePtrAndFlags & VALUE_PTR_MASK);
	}

	template<class ValueType, class AllocatorType>
	inline TUAFCoWRef<ValueType, AllocatorType>& FindMutableCoWRef(TUAFCoWRef<ValueType, AllocatorType>& Arg0, TUAFCoWRef<ValueType, AllocatorType>& Arg1)
	{
		if (Arg0.IsMutable())
		{
			return Arg0;
		}

		if (Arg1.IsMutable())
		{
			return Arg1;
		}

		if (Arg0.IsUnique())
		{
			Arg0.ForceMutable();
			return Arg0;
		}

		Arg1.ForceMutable();
		return Arg1;
	}
}
