// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConstExprUID.h"

#define UE_API UAFANIMNODE_API

// If you wish to use a custom type with a stack when evaluating AnimOps, you'll have to enable
// its usage through this macro to ensure that GetStackTypeID<T>() is declared for your type.
// Use this macro inside the UE::UAF namespace
#define UE_UAF_REGISTER_STACK_TYPE(Type) \
	template<> \
	[[nodiscard]] constexpr uint32 GetStackTypeID<Type>() \
	{ \
		return UE::HashStringFNV1a32(#Type); \
	}

namespace UE::UAF
{
	// Helper function that returns a UID for the specified stack element type
	// @see UE_UAF_REGISTER_STACK_TYPE
	template<typename Type>
	[[nodiscard]] constexpr uint32 GetStackTypeID()
	{
		static_assert(false, "Not implemented! See UE_UAF_REGISTER_STACK_TYPE for details and register your type");
		return 0;
	}

	/*
	 * FUAFStackName
	 *
	 * A small struct to wrap an FName and cache its type hash.
	 */
	class FUAFStackName final
	{
	public:
		// Implicitly coerce or construct a stack name from an FName
		FUAFStackName(const FName& InName)
			: Name(InName)
			, NameHash(GetTypeHash(InName))
		{
		}

		// Returns the stack name
		[[nodiscard]] const FName& GetName() const;

		// Returns the stack name hash
		[[nodiscard]] uint32 GetNameHash() const;

	private:
		FName Name;
		uint32 NameHash = 0;
	};

	/*
	 * FUAFStack
	 *
	 * A simple stack implementation that supports an indirect allocator for its entries.
	 * Type safety is enforced at runtime and compile time where possible.
	 * 
	 * In order to use a stack with a new type, make sure that the type has been registered with
	 * the following macro: UE_UAF_REGISTER_STACK_TYPE.
	 */
	class FUAFStack
	{
	public:
		// Virtual destructor to simplify destruction
		virtual ~FUAFStack() = default;

		// Returns the stack name
		[[nodiscard]] const FName& GetName() const;

		// Returns the number of stack entries
		[[nodiscard]] uint32 Num() const;

		// Returns whether or not the stack is empty
		[[nodiscard]] bool IsEmpty() const;

		// Returns the stack type ID
		[[nodiscard]] uint32 GetTypeID() const;

	protected:
		struct FEntryHeader
		{
			// The top entry at the time this new entry was pushed
			FEntryHeader* PrevTop = nullptr;

			// The allocated size of this entry, includes sizeof(FEntry) and any required alignment
			uint32 Size = 0;

			// The entry start offset from 'this'
			uint32 Offset = 0;

			// Actual entry memory follows

			[[nodiscard]] void* GetValuePtr();
			[[nodiscard]] const void* GetValuePtr() const;
		};

		FUAFStack(const FUAFStackName& Name, uint32 TypeID);
		[[nodiscard]] UE_API const FEntryHeader* PeekImpl(uint32 OffsetFromTop) const;

		// The top entry on the stack
		FEntryHeader* Top = nullptr;

		// The name of this stack
		FUAFStackName Name;

		// How many entries we have on the stack
		uint32 NumEntries = 0;

		// Type ID used for runtime checks to make sure we always read/write using the same type
		uint32 TypeID = 0;
	};

	/*
	 * TUAFStack
	 *
	 * A simple stack implementation that supports an indirect allocator for its entries.
	 * Type safety is enforced at runtime and compile time where possible.
	 *
	 * In order to use a stack with a new type, make sure that the type has been registered with
	 * the following macro: UE_UAF_REGISTER_STACK_TYPE.
	 */
	template<typename ValueType>
	class TUAFStack : public FUAFStack
	{
	public:
		// Constructs a named stack
		TUAFStack(const FUAFStackName& Name);
		virtual ~TUAFStack() override;

		// Pushes a new value onto the stack
		void Push(ValueType&& Value);

		// Pops and returns a value from the stack
		[[nodiscard]] ValueType Pop();

		// Peeks at an element with the specified offset from the top and returns it
		[[nodiscard]] const ValueType* Peek(uint32 OffsetFromTop = 0) const;
		[[nodiscard]] ValueType* PeekMutable(uint32 OffsetFromTop = 0);
	};

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline const FName& FUAFStackName::GetName() const
	{
		return Name;
	}

	inline uint32 FUAFStackName::GetNameHash() const
	{
		return NameHash;
	}

	inline const FName& FUAFStack::GetName() const
	{
		return Name.GetName();
	}

	inline uint32 FUAFStack::Num() const
	{
		return NumEntries;
	}

	inline bool FUAFStack::IsEmpty() const
	{
		return Top == nullptr;
	}

	inline uint32 FUAFStack::GetTypeID() const
	{
		return TypeID;
	}

	inline FUAFStack::FUAFStack(const FUAFStackName& InName, uint32 InTypeID)
		: Name(InName)
		, TypeID(InTypeID)
	{
	}

	inline void* FUAFStack::FEntryHeader::GetValuePtr()
	{
		return reinterpret_cast<uint8*>(this) + Offset;
	}

	inline const void* FUAFStack::FEntryHeader::GetValuePtr() const
	{
		return reinterpret_cast<const uint8*>(this) + Offset;
	}

	template<typename ValueType>
	inline TUAFStack<ValueType>::TUAFStack(const FUAFStackName& Name)
		: FUAFStack(Name, GetStackTypeID<ValueType>())
	{
	}

	template<typename ValueType>
	inline TUAFStack<ValueType>::~TUAFStack()
	{
		FEntryHeader* EntryHeader = Top;
		while (EntryHeader != nullptr)
		{
			FEntryHeader* PrevEntryHeader = EntryHeader->PrevTop;

			static_cast<ValueType*>(EntryHeader->GetValuePtr())->~ValueType();
			FMemory::Free(EntryHeader);

			EntryHeader = PrevEntryHeader;
		}
	}

	template<typename ValueType>
	inline void TUAFStack<ValueType>::Push(ValueType&& Value)
	{
		checkf(TypeID == GetStackTypeID<ValueType>(), TEXT("Type mismatch! This stack is being queried with a different type than it was created with."));

		constexpr uint32 MinAlign = alignof(ValueType) < alignof(FEntryHeader) ? alignof(FEntryHeader) : alignof(ValueType);
		const uint32 EntryOffset = Align(sizeof(FEntryHeader), alignof(ValueType));
		const uint32 AllocationSize = EntryOffset + sizeof(ValueType);

		FEntryHeader* EntryHeader = static_cast<FEntryHeader*>(FMemory::Malloc(AllocationSize, MinAlign));
		EntryHeader->PrevTop = Top;
		EntryHeader->Size = AllocationSize;
		EntryHeader->Offset = EntryOffset;

		// We are pushing an immutable version of our value, move its contents to the stack, we'll own it
		ValueType* ValuePtr = static_cast<ValueType*>(EntryHeader->GetValuePtr());
		new(ValuePtr) ValueType(MoveTemp(Value));

		Top = EntryHeader;
		NumEntries++;
	}

	template<typename ValueType>
	inline ValueType TUAFStack<ValueType>::Pop()
	{
		checkf(TypeID == GetStackTypeID<ValueType>(), TEXT("Type mismatch! This stack is being queried with a different type than it was created with."));
		checkf(Top != nullptr, TEXT("Attempting to pop an element from an empty stack."));

		FEntryHeader* EntryHeader = Top;
		Top = EntryHeader->PrevTop;
		NumEntries--;

		ValueType* ValuePtr = static_cast<ValueType*>(EntryHeader->GetValuePtr());

		ValueType Result = MoveTemp(*ValuePtr);

		ValuePtr->~ValueType();
		FMemory::Free(EntryHeader);

		return Result;
	}

	template<typename ValueType>
	inline const ValueType* TUAFStack<ValueType>::Peek(uint32 OffsetFromTop) const
	{
		checkf(TypeID == GetStackTypeID<ValueType>(), TEXT("Type mismatch! This stack is being queried with a different type than it was created with."));
		checkf(Top != nullptr, TEXT("Attempting to peek at an element from an empty stack."));

		return static_cast<const ValueType*>(PeekImpl(OffsetFromTop)->GetValuePtr());
	}

	template<typename ValueType>
	inline ValueType* TUAFStack<ValueType>::PeekMutable(uint32 OffsetFromTop)
	{
		checkf(TypeID == GetStackTypeID<ValueType>(), TEXT("Type mismatch! This stack is being queried with a different type than it was created with."));
		checkf(Top != nullptr, TEXT("Attempting to peek at an element from an empty stack."));

		return static_cast<ValueType*>(const_cast<FEntryHeader*>(PeekImpl(OffsetFromTop))->GetValuePtr());
	}
}

#undef UE_API
