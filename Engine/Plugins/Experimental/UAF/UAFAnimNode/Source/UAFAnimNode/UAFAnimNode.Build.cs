// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFAnimNode : ModuleRules
	{
		public UAFAnimNode(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;

			PublicDependencyModuleNames.AddRange(new string[] { "UAF" });
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"RigVM",
					"ControlRig",
					"Engine",
					"UAFAnimGraph",
					"TraceLog",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UAFUncookedOnly",
						"UAFEditor",
						"WorkspaceEditor",
					}
				);
			}

		}
	}
}
