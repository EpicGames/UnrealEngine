<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * UAF AnimNode Visualizers
  *
  -->

  <Type Name="UE::UAF::TUAFCoWRef&lt;*,*&gt;">
    <SmartPointer Usage="Minimal">($T1*)(ValuePtrAndFlags &amp; VALUE_PTR_MASK)</SmartPointer>
    <DisplayString Condition="ValuePtrAndFlags == 0">nullptr</DisplayString>
    <DisplayString Condition="(ValuePtrAndFlags &amp; (IS_MUTABLE_FLAG_BIT | OWNS_MEMORY_FLAG_BIT)) != 0">MutableOwned={($T1*)(ValuePtrAndFlags &amp; VALUE_PTR_MASK)}</DisplayString>
    <DisplayString Condition="(ValuePtrAndFlags &amp; IS_MUTABLE_FLAG_BIT) != 0">MutableRef={($T1*)(ValuePtrAndFlags &amp; VALUE_PTR_MASK)}</DisplayString>
    <DisplayString Condition="(ValuePtrAndFlags &amp; OWNS_MEMORY_FLAG_BIT) != 0">ImmutableOwned={($T1*)(ValuePtrAndFlags &amp; VALUE_PTR_MASK)}</DisplayString>
    <DisplayString>ImmutableRef={($T1*)(ValuePtrAndFlags &amp; VALUE_PTR_MASK)}</DisplayString>
    <Expand>
      <ExpandedItem Condition="ValuePtrAndFlags">*($T1*)(ValuePtrAndFlags &amp; VALUE_PTR_MASK)</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::UAF::TUAFStack&lt;*&gt;">
    <DisplayString Condition="Top == nullptr">Empty</DisplayString>
    <DisplayString>Num={NumEntries}</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="EntryHeader" InitialValue="Top"/>
        <Variable Name="EntryIndex" InitialValue="NumEntries - 1"/>
        <Size>NumEntries</Size>
        <Loop Condition="EntryHeader != nullptr">
          <Item Name="{EntryIndex}">*($T1*)((const uint8*)EntryHeader + EntryHeader-&gt;Offset)</Item>
          <Exec>EntryHeader = EntryHeader-&gt;PrevTop</Exec>
          <Exec>EntryIndex--</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FUAFNullAnimOp">
    <DisplayString>Null</DisplayString>
  </Type>

  <Type Name="UE::UAF::TUAFAnimOpTuple&lt;*&gt;">
    <Expand>
      <ExpandedItem>AnimOps</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FUAFAnimOpArrayView">
    <Expand>
      <ExpandedItem>AnimOps</ExpandedItem>
    </Expand>
  </Type>
 
</AutoVisualizer>
