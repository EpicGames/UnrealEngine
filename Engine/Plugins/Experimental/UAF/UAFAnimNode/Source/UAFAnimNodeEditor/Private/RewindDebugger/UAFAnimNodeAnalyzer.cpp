// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimNodeAnalyzer.h"

#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"
#include "UAFAnimNodeProvider.h"
#include "TraceServices/Utils.h"
#include "StructUtils/PropertyBag.h"
#include "Serialization/ObjectReader.h"

FUAFAnimNodeAnalyzer::FUAFAnimNodeAnalyzer(TraceServices::IAnalysisSession& InSession, FUAFAnimNodeProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FUAFAnimNodeAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_AnimOpList, "UAFAnimNode", "AnimOpList");
	Builder.RouteEvent(RouteId_AnimNodeUpdate, "UAFAnimNode", "AnimNodeUpdate");
	Builder.RouteEvent(RouteId_AnimNodeValue, "UAFAnimNode", "AnimNodeValue");
}

bool FUAFAnimNodeAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FUAFAnimNodeAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_AnimOpList:
		{
			uint64 OuterObjectId = EventData.GetValue<uint64>("OuterObjectId");
			uint64 GraphInstanceId = EventData.GetValue<uint64>("InstanceId");
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");

			TArrayView<const uint8> ListData = EventData.GetArrayView<uint8>("ListData");

			Provider.AppendAnimOpList(Context.EventTime.AsSeconds(Cycle), RecordingTime, OuterObjectId, GraphInstanceId, ListData);
			break;
		}
		case RouteId_AnimNodeUpdate:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			uint64 RootGraphId = EventData.GetValue<uint64>("RootGraphId");
			uint64 NodeId = EventData.GetValue<uint64>("NodeId");
			uint64 ParentNodeId = EventData.GetValue<uint64>("ParentNodeId");
			float TotalWeight = EventData.GetValue<float>("TotalWeight");

			Provider.AppendAnimNodeUpdate(Context.EventTime.AsSeconds(Cycle), RecordingTime, RootGraphId, NodeId, ParentNodeId, TotalWeight);
			break;			
		}
		case RouteId_AnimNodeValue:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			uint64 RootGraphId = EventData.GetValue<uint64>("RootGraphId");
			uint64 NodeId = EventData.GetValue<uint64>("NodeId");
			uint32 NameId = EventData.GetValue<uint32>("NameId");
			uint8 Type = EventData.GetValue<uint8>("Type");
			uint64 StructType = EventData.GetValue<uint64>("StructType");
			TArrayView<const uint8> Value = EventData.GetArrayView<uint8>("Value");
			Provider.AppendAnimNodeValue(Context.EventTime.AsSeconds(Cycle), RecordingTime, RootGraphId, NodeId, Type, StructType, NameId, Value);
			break;			
		}

	}

	return true;
}
