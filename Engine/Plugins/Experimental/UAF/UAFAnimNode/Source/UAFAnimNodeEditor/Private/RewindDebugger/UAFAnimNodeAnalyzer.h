// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

class FUAFAnimNodeProvider;
namespace TraceServices { class IAnalysisSession; }

class FUAFAnimNodeAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FUAFAnimNodeAnalyzer(TraceServices::IAnalysisSession& InSession, FUAFAnimNodeProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_AnimOpList,
		RouteId_AnimNodeUpdate,
		RouteId_AnimNodeValue,
	};

	TraceServices::IAnalysisSession& Session;
	FUAFAnimNodeProvider& Provider;
};
