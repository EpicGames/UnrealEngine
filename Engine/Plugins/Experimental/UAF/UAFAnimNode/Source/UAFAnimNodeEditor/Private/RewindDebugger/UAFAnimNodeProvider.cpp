// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimNodeProvider.h"

#include "IGameplayProvider.h"
#include "ObjectTrace.h"

namespace TraceServices
{
thread_local FProviderLock::FThreadLocalState GUAFAnimNodeProviderLockState;
}

FName FUAFAnimNodeProvider::ProviderName("UAFAnimNodeProvider");

#define LOCTEXT_NAMESPACE "UAFAnimNodeProvider"

FUAFAnimNodeProvider::FUAFAnimNodeProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

void FUAFAnimNodeProvider::AppendAnimOpList(double ProfileTime, double RecordingTime, uint64 OuterObjectId, uint64 GraphInstanceId, const TArrayView<const uint8>& ListData)
{
	EditAccessCheck();

	EvaluationGraphs.Add(OuterObjectId, GraphInstanceId);

	const TSharedRef<FAnimOpData>* Data = AnimOpData.Find(GraphInstanceId);
	if (Data == nullptr)
	{
		TSharedRef<FAnimOpData> NewData = MakeShared<FAnimOpData>(Session);
		NewData->GraphInstanceId = GraphInstanceId;
		AnimOpData.Add(GraphInstanceId, NewData);
		Data = AnimOpData.Find(GraphInstanceId);
	}

	if (!ListData.IsEmpty())
	{
		(*Data)->AnimOpTimeline.AppendEvent(ProfileTime, TArray<uint8>(ListData));
	}
}

void FUAFAnimNodeProvider::AppendAnimNodeUpdate(double ProfileTime, double RecordingTime, uint64 RootGraphId, uint64 NodeId, uint64 ParentNodeId, float TotalWeight)
{
	EditAccessCheck();
    
	const TSharedRef<FAnimNodeUpdateTimelineData>* Data = AnimNodeUpdateTimelineData.Find(RootGraphId);
	if (Data == nullptr)
	{
		TSharedRef<FAnimNodeUpdateTimelineData> NewData = MakeShared<FAnimNodeUpdateTimelineData>(Session);
		NewData->GraphInstanceId = RootGraphId;
		AnimNodeUpdateTimelineData.Add(RootGraphId, NewData);
		Data = AnimNodeUpdateTimelineData.Find(RootGraphId);
	}

	(*Data)->NodeUpdateTimeline.AppendEvent(ProfileTime, { ProfileTime, RecordingTime, ParentNodeId, NodeId, TotalWeight });
}


void FUAFAnimNodeProvider::AppendAnimNodeValue(double ProfileTime, double RecordingTime, uint64 RootGraphId, uint64 NodeId, uint8 Type, uint64 StructType, uint32 NameId, TConstArrayView<uint8> Value)
{
	EditAccessCheck();
    
	const TSharedRef<FAnimNodeValueTimelineData>* Data = AnimNodeValueTimelineData.Find(RootGraphId);
	if (Data == nullptr)
	{
		TSharedRef<FAnimNodeValueTimelineData> NewData = MakeShared<FAnimNodeValueTimelineData>(Session);
		NewData->GraphInstanceId = RootGraphId;
		AnimNodeValueTimelineData.Add(RootGraphId, NewData);
		Data = AnimNodeValueTimelineData.Find(RootGraphId);
	}

	FAnimNodeValueData ValueData;

	ValueData.ProfileTime = ProfileTime;
	ValueData.RecordingTime = RecordingTime;
	ValueData.NodeId = NodeId;
	ValueData.NameId = NameId;
	ValueData.Type = Type;
	if (StructType != 0)
	{
		if (const IGameplayProvider* GameplayProvider = Session.ReadProvider<IGameplayProvider>("GameplayProvider"))
		{
			GameplayProvider->BeginRead();
			const FObjectInfo& StructInfo = GameplayProvider->GetObjectInfo(StructType);
			GameplayProvider->EndRead();
			ValueData.StructType = FindObject<UScriptStruct>(nullptr, StructInfo.PathName);
		}
	}
	ValueData.Value = Value;

	(*Data)->NodeValueTimeline.AppendEvent(ProfileTime, ValueData);
}

const FAnimOpData* FUAFAnimNodeProvider::GetAnimOpData(uint64 GraphInstanceId) const
{
	ReadAccessCheck();

	const TSharedRef<FAnimOpData>* Data = AnimOpData.Find(GraphInstanceId);
	if (Data)
	{
		return &**Data;
	}
	return nullptr;
}

const FAnimNodeUpdateTimelineData* FUAFAnimNodeProvider::GetAnimNodeTimelineData(uint64 GraphInstanceId) const
{
	ReadAccessCheck();
	
	const TSharedRef<FAnimNodeUpdateTimelineData>* Data = AnimNodeUpdateTimelineData.Find(GraphInstanceId);
	if (Data)
	{
		return &Data->Get();
	}

	return nullptr;
}

const FAnimNodeValueTimelineData* FUAFAnimNodeProvider::GetAnimNodeValueTimelineData(uint64 GraphInstanceId) const
{
	ReadAccessCheck();
	
	const TSharedRef<FAnimNodeValueTimelineData>* Data = AnimNodeValueTimelineData.Find(GraphInstanceId);
	if (Data)
	{
		return &Data->Get();
	}

	return nullptr;
}

void FUAFAnimNodeProvider::EnumerateEvaluationGraphs(uint64 OuterObjectId, TFunctionRef<void(uint64 GraphInstanceId)> Callback) const
{
	ReadAccessCheck();

	TArray<uint64> GraphIds;
	EvaluationGraphs.MultiFind(OuterObjectId, GraphIds);
	
	for (uint64 GraphId : GraphIds)
	{
		Callback(GraphId);
	}
}

#undef LOCTEXT_NAMESPACE
