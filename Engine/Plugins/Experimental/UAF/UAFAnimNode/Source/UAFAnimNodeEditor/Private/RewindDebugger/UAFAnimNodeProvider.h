// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Model/PointTimeline.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Model/IntervalTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"

namespace TraceServices
{
	class IAnalysisSession;
	extern thread_local FProviderLock::FThreadLocalState GUAFAnimNodeProviderLockState;
}


struct FSmallPageSettings
{
	enum
	{
		EventsPerPage = 4096
	};
};

struct FAnimOpData
{
	FAnimOpData(TraceServices::IAnalysisSession& Session)
		: GraphInstanceId(0),
		AnimOpTimeline(Session.GetLinearAllocator())

	{
	}

	uint64 GraphInstanceId;
	TraceServices::TPointTimeline<TArray<uint8>, FSmallPageSettings> AnimOpTimeline;
};

struct FAnimNodeUpdateData
{
	double ProfileTime;
	double RecordingTime;
	uint64 ParentNodeId;
	uint64 NodeId;
	float TotalWeight;
};

struct FAnimNodeUpdateTimelineData
{
	FAnimNodeUpdateTimelineData(TraceServices::IAnalysisSession& Session)
		: GraphInstanceId(0),
		NodeUpdateTimeline(Session.GetLinearAllocator())
	{
	}

	uint64 GraphInstanceId;
	TraceServices::TPointTimeline<FAnimNodeUpdateData,FSmallPageSettings> NodeUpdateTimeline;
};

struct FAnimNodeValueData
{
	double ProfileTime;
	double RecordingTime;
	uint64 NodeId;
	uint32 NameId;
	uint8 Type;
	TArray<uint8, TInlineAllocator<8>> Value;
	const UScriptStruct* StructType = nullptr;
};


struct FAnimNodeValueTimelineData
{
	FAnimNodeValueTimelineData(TraceServices::IAnalysisSession& Session)
		: GraphInstanceId(0),
		NodeValueTimeline(Session.GetLinearAllocator())
	{
	}

	uint64 GraphInstanceId;
	TraceServices::TPointTimeline<FAnimNodeValueData,FSmallPageSettings> NodeValueTimeline;
};

class FUAFAnimNodeProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider
{
public:
	static FName ProviderName;

	FUAFAnimNodeProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GUAFAnimNodeProviderLockState);
	}

	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GUAFAnimNodeProviderLockState);
	}

	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GUAFAnimNodeProviderLockState);
	}

	//~ IEditableProvider (edit lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GUAFAnimNodeProviderLockState);
	}

	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GUAFAnimNodeProviderLockState);
	}

	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GUAFAnimNodeProviderLockState);
	}

	const FAnimOpData* GetAnimOpData(uint64 GraphInstanceId) const;
	const FAnimNodeUpdateTimelineData* GetAnimNodeTimelineData(uint64 GraphInstanceId) const;
	const FAnimNodeValueTimelineData* GetAnimNodeValueTimelineData(uint64 GraphInstanceId) const;
	
	void AppendAnimOpList(double ProfileTime, double RecordingTime, uint64 ObjectInstanceId, uint64 GraphInstanceId, const TArrayView<const uint8>& ListData);
	void AppendAnimNodeUpdate(double ProfileTime, double RecordingTime, uint64 RootGraphId, uint64 NodeId, uint64 ParentNodeId, float TotalWeight);
	void AppendAnimNodeValue(double ProfileTime, double RecordingTime, uint64 RootGraphId, uint64 NodeId, uint8 TypeId, uint64 StructType, uint32 NameId, TConstArrayView<uint8> Value);
	
	void EnumerateEvaluationGraphs(uint64 OuterObjectId, TFunctionRef<void(uint64 GraphInstanceId)> Callback) const;
	
private:
	TraceServices::IAnalysisSession& Session;
	mutable TraceServices::FProviderLock Lock;

	TMap<uint64, TSharedRef<FAnimOpData>> AnimOpData;
	TMap<uint64, TSharedRef<FAnimNodeUpdateTimelineData>> AnimNodeUpdateTimelineData;
	TMap<uint64, TSharedRef<FAnimNodeValueTimelineData>> AnimNodeValueTimelineData;

	// map from object id to a list of graph instance ids that have evaluation data
	TMultiMap<uint64, uint64> EvaluationGraphs;
};
