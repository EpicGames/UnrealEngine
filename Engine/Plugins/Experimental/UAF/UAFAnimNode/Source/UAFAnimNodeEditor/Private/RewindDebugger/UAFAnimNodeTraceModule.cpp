// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimNodeTraceModule.h"
#include "UAFAnimNodeProvider.h"
#include "UAFAnimNodeAnalyzer.h"

FName FUAFAnimNodeTraceModule::ModuleName("UAFAnimNode");

void FUAFAnimNodeTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("UAFAnimNode");
}

void FUAFAnimNodeTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FUAFAnimNodeProvider> Provider = MakeShared<FUAFAnimNodeProvider>(InSession);
	InSession.AddProvider(FUAFAnimNodeProvider::ProviderName, Provider);

	InSession.AddAnalyzer(new FUAFAnimNodeAnalyzer(InSession, *Provider));
}

void FUAFAnimNodeTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("UAFAnimNode"));
}

void FUAFAnimNodeTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

