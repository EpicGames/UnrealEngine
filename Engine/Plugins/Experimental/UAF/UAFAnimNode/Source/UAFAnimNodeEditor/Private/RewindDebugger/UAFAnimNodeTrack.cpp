// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimNodeTrack.h"

#include "Common/ProviderLock.h"
#include "UAFAnimNodeProvider.h"
#include "DetailWidgetRow.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Animation/AnimSequence.h"
#include "UAF/AnimOps/UAFDecompressAnimSequenceAnimOp.h"
#include "Component/AnimNextComponent.h"
#include "TraceServices/Model/Frames.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimNodeCore/UAFTransitionNodeData.h"
#include "IAnimationProvider.h"
#include "ObjectAsTraceIdProxyArchiveReader.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFAnimNodeTrack)

#define LOCTEXT_NAMESPACE "UAFAnimNodeTrack"

namespace UE::UAF::Editor
{

FName FUAFAnimNodeTrackCreator::GetTargetTypeNameInternal() const
{
	return FUAFAnimNodeData::StaticStruct()->GetFName();
}

FName FUAFTransitionTrackCreator::GetTargetTypeNameInternal() const
{
	return FUAFTransitionNodeData::StaticStruct()->GetFName();
}

FText FUAFAnimNodeTrack::GetDisplayNameInternal() const
{
	// return empoty display name to inherit the object track display name
	return FText();
}

void FUAFAnimNodeTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({ FUAFAnimNodeTrack::TrackName, LOCTEXT("UAFAnimNode", "UAF Anim Node")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FUAFAnimNodeTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FUAFAnimNodeTrack>(InObjectId.GetMainId());
}

FUAFAnimNodeTrack::FUAFAnimNodeTrack(uint64 InObjectId)
	: NodeId(InObjectId)
{
}

FUAFAnimNodeTrack::~FUAFAnimNodeTrack()
{
}

TSharedPtr<SCurveTimelineView::FTimelineCurveData> FUAFAnimNodeTrack::GetCurveData() const
{
	if (!CurveData.IsValid())
	{
		CurveData = MakeShared<SCurveTimelineView::FTimelineCurveData>();
	}
	
	CurvesUpdateRequested++;
	
	return CurveData;
}

static FLinearColor MakeBlendWeightCurveColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

TSharedPtr<SWidget> FUAFAnimNodeTrack::GetTimelineViewInternal()
{
	FLinearColor FillColor;
	FLinearColor CurveColor;
	FLinearColor SelectedColor;
	
	FillColor = MakeBlendWeightCurveColor(CityHash32(reinterpret_cast<char*>(&NodeId), 8)); // todo make this use AssetId when available
	FillColor.A = 0.5f;
	CurveColor = FillColor;
	CurveColor.R *= 0.5;
	CurveColor.G *= 0.5;
	CurveColor.B *= 0.5;
	SelectedColor = CurveColor;

	TSharedPtr<SCurveTimelineView> CurveTimelineView = SNew(SCurveTimelineView)
		.TrackName(GetDisplayNameInternal())
		.FillColor(FillColor)
		.CurveColor_Lambda([CurveColor, SelectedColor,  this]() { return GetIsSelected() ? SelectedColor : CurveColor; })
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.RenderFill(true)
		.CurveData_Raw(this, &FUAFAnimNodeTrack::GetCurveData);

	CurveTimelineView->SetFixedRange(0, 1);

	return CurveTimelineView;
}

TSharedPtr<SWidget> FUAFAnimNodeTrack::GetDetailsViewInternal()
{
	if (!DetailsObject)
	{
		DetailsObject = TStrongObjectPtr(NewObject<UUAFAnimNodeTrackDetailsObject>());
	}

	if (!DetailsView)
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
        FDetailsViewArgs DetailsViewArgs;
        DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
        DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(DetailsObject.Get());
	}

	return DetailsView;
}
	
bool FUAFAnimNodeTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUAFAnimNodeTrack::UpdateInternal);

	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	check(RewindDebugger);
	const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession();
	if (Session == nullptr)
	{
		return false;
	}

	const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
	const IAnimationProvider* AnimationProvider = Session->ReadProvider<IAnimationProvider>("AnimationProvider");
	const FUAFAnimNodeProvider* UAFAnimNodeProvider = Session->ReadProvider<FUAFAnimNodeProvider>("UAFAnimNodeProvider");
	if (GameplayProvider == nullptr
		|| AnimationProvider == nullptr
		|| UAFAnimNodeProvider == nullptr)
	{
		return false;
	}

	const double CurrentScrubTime = RewindDebugger->CurrentTraceTime();

	TraceServices::FFrame MarkerFrame;
	{
		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*RewindDebugger->GetAnalysisSession());
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);
		if (!FrameProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, CurrentScrubTime, MarkerFrame))
		{
			return false;
		}
	}

	if (GraphId == 0)
	{
		TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
		const FObjectInfo& Info = GameplayProvider->GetObjectInfo(NodeId);
		GraphId = Info.GetOuterId().GetMainId();
		if (GraphId == 0)
		{
			return false;
		}
	}

	TraceServices::FProviderReadScopeLock UAFAnimNodeProviderReadScope(*UAFAnimNodeProvider);
	if (const FAnimNodeUpdateTimelineData* Data = UAFAnimNodeProvider->GetAnimNodeTimelineData(GraphId))
	{
		if (PreviousScrubTime != CurrentScrubTime && DetailsObject)
		{
			PreviousScrubTime = CurrentScrubTime;

			// clear properties first so we don't see old data
			DetailsObject->Properties.Reset();

			Data->NodeUpdateTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime,
				[this](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeUpdateData& NodeUpdateData)
				{
					if (NodeUpdateData.NodeId == NodeId)
					{
						static FName PropertyName("TotalWeight");
						FPropertyBagPropertyDesc PropertyDesc(PropertyName, EPropertyBagPropertyType::Float);
						DetailsObject->Properties.AddProperties({ PropertyDesc });
						DetailsObject->Properties.SetValueFloat(PropertyName, NodeUpdateData.TotalWeight);
						return TraceServices::EEventEnumerate::Stop;
					}
					return TraceServices::EEventEnumerate::Continue;
				});

			if (auto ValueData = UAFAnimNodeProvider->GetAnimNodeValueTimelineData(GraphId))
			{
				TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
				TraceServices::FProviderReadScopeLock AnimationProviderReadScope(*AnimationProvider);

				ValueData->NodeValueTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime,
					[GameplayProvider, AnimationProvider, this](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeValueData& NodeValueData)
					{
						if (NodeValueData.NodeId == NodeId)
						{
							FName PropertyName = AnimationProvider->GetName(NodeValueData.NameId);
							
							FPropertyBagPropertyDesc PropertyDesc(PropertyName, static_cast<EPropertyBagPropertyType>(NodeValueData.Type), UObject::StaticClass());
							if (NodeValueData.StructType)
							{
								PropertyDesc.ValueTypeObject = NodeValueData.StructType;
							}
							
							DetailsObject->Properties.AddProperties({ PropertyDesc });

							switch (static_cast<EPropertyBagPropertyType>(NodeValueData.Type))
							{
							case EPropertyBagPropertyType::Bool:
								if (ensure(NodeValueData.Value.Num() == sizeof(bool)))
								{
									DetailsObject->Properties.SetValueBool(PropertyName, *reinterpret_cast<const bool*>(NodeValueData.Value.GetData()));
								}
								break;
							case EPropertyBagPropertyType::Double:
								if (ensure(NodeValueData.Value.Num() == sizeof(double)))
								{
									DetailsObject->Properties.SetValueDouble(PropertyName, *reinterpret_cast<const double*>(NodeValueData.Value.GetData()));
								}
								break;
							case EPropertyBagPropertyType::Object:
							{
								if (ensure(NodeValueData.Value.Num() == sizeof(uint64)))
								{
									UObject* Asset = nullptr;
									const uint64 ObjectId = *reinterpret_cast<const uint64*>(NodeValueData.Value.GetData());
									const FObjectInfo& AssetInfo = GameplayProvider->GetObjectInfo(ObjectId);
									const FString PackagePathString = FPackageName::ObjectPathToPackageName(FString(AssetInfo.PathName));

									UPackage* Package = LoadPackage(nullptr, ToCStr(PackagePathString), LOAD_NoRedirects);
									if (Package)
									{
										Package->FullyLoad();

										FString AssetName = FPaths::GetBaseFilename(AssetInfo.PathName);
										Asset = FindObject<UObject>(Package, *AssetName);
									}
									else
									{
										// fallback for unsaved assets
										Asset = FindObject<UObject>(nullptr, AssetInfo.PathName);
									}

									DetailsObject->Properties.SetValueObject(PropertyName, Asset);
								}
								break;
							}
							case EPropertyBagPropertyType::Struct:
							{
								if (NodeValueData.StructType)
								{
									TArray<uint8> DeserializedStruct;
									DeserializedStruct.SetNum(NodeValueData.StructType->GetStructureSize());
									NodeValueData.StructType->InitializeStruct(DeserializedStruct.GetData());
									TArray<uint8> ValueArray(NodeValueData.Value.GetData(), NodeValueData.Value.Num());
									FMemoryReader StructReader(ValueArray);
									FObjectAsTraceIdProxyArchiveReader Reader(StructReader, GameplayProvider);
									NodeValueData.StructType->SerializeBin(Reader, DeserializedStruct.GetData());
								
									DetailsObject->Properties.SetValueStruct(PropertyName,FConstStructView(NodeValueData.StructType, DeserializedStruct.GetData()));
									
									NodeValueData.StructType->DestroyStruct(DeserializedStruct.GetData());
								}
								break;
							}
							default:
								break;
							};
						}
						return TraceServices::EEventEnumerate::Continue;
					});
			}
		}

		if (CurvesUpdateRequested > 10)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FUAFAnimNodeTrack::UpdateCurvePointsInternal);
			CurveData->Points.SetNum(0, EAllowShrinking::No);

			const TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
			const double StartTime = TraceTimeRange.GetLowerBoundValue();
			const double EndTime = TraceTimeRange.GetUpperBoundValue();

			Data->NodeUpdateTimeline.EnumerateEvents(StartTime, EndTime, [this](double InStartTime, double InEndTime, uint32 InDepth, const FAnimNodeUpdateData& NodeUpdateData)
				{
					if (NodeUpdateData.NodeId == NodeId)
					{
						CurveData->Points.Add({ NodeUpdateData.RecordingTime, NodeUpdateData.TotalWeight });
					}

					return TraceServices::EEventEnumerate::Continue;
				});

			CurvesUpdateRequested = 0;
		}
	}

	return false;
}

bool FUAFAnimNodeTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return true;
}
}

#undef LOCTEXT_NAMESPACE
