// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "UAF/AnimOpCore/UAFInstancedAnimOpList.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"
#include "IDetailsView.h"
#include "IPropertyTypeCustomization.h"
#include "StructUtils/PropertyBag.h"
#include "SCurveTimelineView.h"

#include "UAFAnimNodeTrack.generated.h"


UCLASS()
class UUAFAnimNodeTrackDetailsObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Details, meta=(ShowOnlyInnerProperties))
	FInstancedPropertyBag Properties;
};

namespace UE::UAF::Editor
{

class FUAFAnimNodeTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FUAFAnimNodeTrack(uint64 InObjectId);
	virtual ~FUAFAnimNodeTrack();
	
	static constexpr const TCHAR* TrackName = TEXT("AnimNodeTrack");
	
private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return TrackName; }
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override { return NodeId; }

	TSharedPtr<IDetailsView> DetailsView;
	FSlateIcon Icon;
	uint64 NodeId;
	uint64 GraphId = 0;
	double PreviousScrubTime = -1.0;
	TStrongObjectPtr<UUAFAnimNodeTrackDetailsObject> DetailsObject;

	TSharedPtr<SCurveTimelineView::FTimelineCurveData> GetCurveData() const;
	mutable TSharedPtr<SCurveTimelineView::FTimelineCurveData> CurveData;
	mutable int CurvesUpdateRequested = 0;
};

class FUAFAnimNodeTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override { return FUAFAnimNodeTrack::TrackName; }
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool IsCreatingPrimaryChildTrackInternal() const override { return true; }
};

class FUAFTransitionTrackCreator : public FUAFAnimNodeTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
};

}