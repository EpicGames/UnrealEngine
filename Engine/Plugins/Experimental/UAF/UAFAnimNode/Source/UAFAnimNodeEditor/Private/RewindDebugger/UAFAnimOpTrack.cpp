// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimOpTrack.h"

#include "Common/ProviderLock.h"
#include "UAFAnimNodeProvider.h"
#include "Editor.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ObjectAsTraceIdProxyArchiveReader.h"
#include "Component/AnimNextComponent.h"
#include "Serialization/ObjectReader.h"
#include "TraceServices/Model/Frames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFAnimOpTrack)


#define LOCTEXT_NAMESPACE "UAFAnimOpTrack"

namespace UE::UAF::Editor
{

static const FName UAFModulesName("UAFModules");

 FName FUAFAnimOpTrackCreator::GetTargetTypeNameInternal() const
 {
	return UUAFComponent::StaticClass()->GetFName();
}

FText FUAFAnimOpTrack::GetDisplayNameInternal() const
{
 	return NSLOCTEXT("RewindDebugger", "UAFAnimOpTrackName", "AnimOps");
}

void FUAFAnimOpTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({ UAFModulesName, LOCTEXT("UAFSystems", "UAF Systems")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FUAFAnimOpTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FUAFAnimOpTrack>(InObjectId.GetMainId());
}

		
FUAFAnimOpTrack::FUAFAnimOpTrack(uint64 InObjectId) :
	ObjectId(InObjectId)
{
	Initialize();
}
	
FUAFAnimOpTrack::FUAFAnimOpTrack(uint64 InObjectId, uint64 InInstanceId) :
   	ObjectId(InObjectId),
	InstanceId(InInstanceId)
{
   	Initialize();
}
	
	
FUAFAnimOpTrack::~FUAFAnimOpTrack()
{
 	if (UAnimOpListDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get())
   	{
   		DetailsObject->ClearFlags(RF_Standalone);
   	}
}
	
void FUAFAnimOpTrack::Initialize()
{
	ExistenceRange = MakeShared<SEventTimelineView::FTimelineEventData>();
	ExistenceRange->EventWindows.Add({0,0, GetDisplayNameInternal(), GetDisplayNameInternal(), FLinearColor(0.1f,0.15f,0.11f)});
	
   	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
   	FDetailsViewArgs DetailsViewArgs;
   	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
   	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InitializeDetailsObject();
}
	
UAnimOpListDetailsObject* FUAFAnimOpTrack::InitializeDetailsObject()
{
	UAnimOpListDetailsObject* DetailsObject = NewObject<UAnimOpListDetailsObject>();
	DetailsObject->SetFlags(RF_Standalone);
	DetailsObjectWeakPtr = MakeWeakObjectPtr(DetailsObject);
	DetailsView->SetObject(DetailsObject);
 	return DetailsObject;
}

bool FUAFAnimOpTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUAFAnimOpTrack::UpdateInternal);

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TRange<double> ViewRange = RewindDebugger->GetCurrentViewRange();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	constexpr bool bChanged = false;

	if (const FUAFAnimNodeProvider* UAFAnimNodeProvider = AnalysisSession->ReadProvider<FUAFAnimNodeProvider>("UAFAnimNodeProvider"))
	{
		const double CurrentScrubTime = IRewindDebugger::Instance()->CurrentTraceTime();

		UAnimOpListDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get();
		if (DetailsObject == nullptr)
		{
			// this should not happen unless the object was garbage collected (which should not happen since it's marked as Standalone)
			Initialize();
		}

		if (InstanceId == 0)
		{
			TraceServices::FProviderReadScopeLock UAFProviderReadScope(*UAFAnimNodeProvider);
			UAFAnimNodeProvider->EnumerateEvaluationGraphs(ObjectId, [this](uint64 GraphId)
				{
					InstanceId = GraphId;
				});
		}

		if (InstanceId != 0
			&& PreviousScrubTime != CurrentScrubTime)
		{
			PreviousScrubTime = CurrentScrubTime;

			bool bFrameFound;
			TraceServices::FFrame MarkerFrame;
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
				const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
				bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime, MarkerFrame);
			}

			if (bFrameFound)
			{
				TraceServices::FProviderReadScopeLock UAFProviderReadScope(*UAFAnimNodeProvider);
				if (const FAnimOpData* Data = UAFAnimNodeProvider->GetAnimOpData(InstanceId))
				{
					const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
					TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

					Data->AnimOpTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime,
						[DetailsObject, GameplayProvider](double InStartTime, double InEndTime, uint32 InDepth, const TArray<uint8>& VariableData)
						{
							FMemoryReader Reader(VariableData);
							FObjectAsTraceIdProxyArchiveReader Archive(Reader, GameplayProvider);

							static const FUAFInstancedAnimOpList Defaults;
							FUAFInstancedAnimOpList::StaticStruct()->SerializeItem(Archive, &DetailsObject->AnimOps, &Defaults);

							return TraceServices::EEventEnumerate::Stop;
						});
				}
			}

			// TArray<uint64, TInlineAllocator<32>> CurrentChildren;

			// // update/create child tracks
			// AnimNextProvider->EnumerateChildInstances(InstanceId, [this, &bChanged, &CurrentChildren, &ViewRange](const FDataInterfaceData& ChildData)
			// {
			// 	if (ChildData.StartTime < ViewRange.GetUpperBoundValue() && ChildData.EndTime > ViewRange.GetLowerBoundValue())
			// 	{
			// 		CurrentChildren.Add(ChildData.InstanceId);
			// 		if (!Children.ContainsByPredicate([&ChildData](TSharedPtr<FEvaluationProgramTrack>& Child) { return Child->InstanceId == ChildData.InstanceId; } ))
			// 		{
			// 			Children.Add(MakeShared<FEvaluationProgramTrack>(ObjectId, ChildData.InstanceId));
			// 			bChanged = true;
			// 		}
			// 	}
			// });
			//
			// int32 NumRemoved = Children.RemoveAll([&CurrentChildren](TSharedPtr<FEvaluationProgramTrack>& Child)
			// {
			// 	return !CurrentChildren.Contains(Child->InstanceId);
			// });
			// bChanged |= (NumRemoved > 0);
		}

		// for (auto& Child : Children)
		// {
		// 	bChanged |= Child->Update();
		// }
	}

	return bChanged;
}

bool FUAFAnimOpTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	// TRACE_CPUPROFILER_EVENT_SCOPE(FEvaluationProgramTrack::HasDebugInfoInternal);

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		if (const FUAFAnimNodeProvider* UAFAnimNodeProvider = AnalysisSession->ReadProvider<FUAFAnimNodeProvider>("UAFAnimNodeProvider"))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*UAFAnimNodeProvider);
			uint64 InstanceId = 0;
			UAFAnimNodeProvider->EnumerateEvaluationGraphs(InObjectId.GetMainId(), [&InstanceId](uint64 GraphId)
				{
					InstanceId = GraphId;
				});

			if (const FAnimOpData* Data = UAFAnimNodeProvider->GetAnimOpData(InstanceId))
			{
				return Data->AnimOpTimeline.GetEventCount() != 0;
			}
		}
	}

	// We don't have any AnimOps
	return false;
}


}

#undef LOCTEXT_NAMESPACE
