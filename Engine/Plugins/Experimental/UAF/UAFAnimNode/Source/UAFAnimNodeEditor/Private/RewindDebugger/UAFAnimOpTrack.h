// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "UAF/AnimOpCore/UAFInstancedAnimOpList.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"
#include "IDetailsView.h"
#include "SEventTimelineView.h"

#include "UAFAnimOpTrack.generated.h"

UCLASS()
class UAnimOpListDetailsObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=Details, meta=(ShowOnlyInnerProperties))
	UE::UAF::FUAFInstancedAnimOpList AnimOps;
};

namespace UE::UAF::Editor
{
	
class FUAFAnimOpTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FUAFAnimOpTrack(uint64 InObjectId);
	FUAFAnimOpTrack(uint64 InObjectId, uint64 InstanceId);
	virtual ~FUAFAnimOpTrack();
	
	TSharedPtr<SEventTimelineView::FTimelineEventData> GetExistenceRange() const { return ExistenceRange; }
private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override { return DetailsView; }

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "UAFModule"; }
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override { return ObjectId; }

	void Initialize();
	UAnimOpListDetailsObject* InitializeDetailsObject();

	TSharedPtr<IDetailsView> DetailsView;
	FSlateIcon Icon;
	uint64 ObjectId;
	uint64 InstanceId = 0; 
	double PreviousScrubTime = -1.0;
	TWeakObjectPtr<UAnimOpListDetailsObject> DetailsObjectWeakPtr;
	TSharedPtr<SEventTimelineView::FTimelineEventData> ExistenceRange;
};

#if WITH_ENGINE
class FUAFAnimOpTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override { return "UAFModule"; }
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

}
