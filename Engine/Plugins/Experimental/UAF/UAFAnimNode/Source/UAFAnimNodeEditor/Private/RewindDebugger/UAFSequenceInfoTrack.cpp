// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFSequenceInfoTrack.h"

#include "UAFAnimNodeProvider.h"
#include "Common/ProviderLock.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Animation/AnimSequence.h"
#include "UAF/AnimOps/UAFDecompressAnimSequenceAnimOp.h"
#include "ObjectAsTraceIdProxyArchiveReader.h"
#include "Component/AnimNextComponent.h"
#include "TraceServices/Model/Frames.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Colors/SColorBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFSequenceInfoTrack)

#define LOCTEXT_NAMESPACE "UAFSequenceInfoTrack"

namespace UE::UAF::Editor
{
FName FUAFSequenceInfoTrackCreator::GetTargetTypeNameInternal() const
{
	return UUAFComponent::StaticClass()->GetFName();
}


FText FUAFSequenceInfoTrack::GetDisplayNameInternal() const
{
	return NSLOCTEXT("RewindDebugger", "UAFSequenceInfoTrackName", "SequenceInfo");
}

void FUAFSequenceInfoTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({ FUAFSequenceInfoTrack::TrackName, LOCTEXT("UAFSequenceInfo", "SequenceInfo")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FUAFSequenceInfoTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FUAFSequenceInfoTrack>(InObjectId.GetMainId());
}


void FUAFSequenceTraceInfoCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow,
                                                              IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

// Copied from SAnimNotifyPanel
// Todo: find common place for this to live
static FLinearColor GenerateColorFromName(FName Name)
{
	constexpr uint8 Saturation = 255;
	constexpr uint8 Luminosity = 255;
	const uint8 Hue = static_cast<uint8>(GetTypeHash(Name.ToString())) * 157;
	return FLinearColor::MakeFromHSV8(Hue, Saturation, Luminosity);
}

// Helper for lambdas
const FUAFSequenceTraceInfo* ExtractSequenceTraceInfo(TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	void* StructData = nullptr;
	FPropertyAccess::Result Result = StructPropertyHandle->GetValueData(StructData);
	if (Result != FPropertyAccess::Success)
		return nullptr;

	const FUAFSequenceTraceInfo* SequenceInfo = static_cast<FUAFSequenceTraceInfo*>(StructData);
	return SequenceInfo;
}

void FUAFSequenceTraceInfoCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
                                                                class IDetailChildrenBuilder& StructBuilder,
                                                                IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(StructPropertyHandle->GetProperty()))
	{
		if (StructProperty->Struct == FUAFSequenceTraceInfo::StaticStruct())
		{
			static const FColor TimelineBackground(0xFF575761);
			static const FColor TimelineForeground(0xFFF59F00);
			constexpr float TimelineWidth = 300.f;
			constexpr float TimelineHeight = 20.f;

			const FUAFSequenceTraceInfo* SequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
			if (SequenceInfo == nullptr)
				return;

			FString SequenceName = SequenceInfo->AnimSequence.IsValid() ? SequenceInfo->AnimSequence->GetName() : TEXT("NULL");

			TSharedRef<SCanvas> TimelineCanvas =
				SNew(SCanvas)

				// Before playhead block
				+ SCanvas::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Position(FVector2D::ZeroVector)
				.Size_Lambda( [=]()
				{
					const FUAFSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					if (UpdatedSequenceInfo == nullptr)
						return FVector2D::ZeroVector;
					return FVector2D(TimelineWidth * UpdatedSequenceInfo->CalcAnimTimeRatio(), TimelineHeight);
				} )
				[
					SNew(SColorBlock)
					.Color(TimelineForeground)
				]

				// After playhead block
				+ SCanvas::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Position_Lambda([=]()
				{
					const FUAFSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					if (UpdatedSequenceInfo == nullptr)
						return FVector2D::ZeroVector;
					
					return FVector2D(UpdatedSequenceInfo->CalcAnimTimeRatio() * TimelineWidth, 0.f);
				})
				.Size_Lambda([=]()
				{
					const FUAFSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					if (UpdatedSequenceInfo == nullptr)
						return FVector2D::ZeroVector;
					
					return FVector2D(TimelineWidth * (1.0f - UpdatedSequenceInfo->CalcAnimTimeRatio()), TimelineHeight);
				})
				[
					SNew(SColorBlock)
					.Color(TimelineBackground)
				]

				// Time info
				+ SCanvas::Slot()
				.Position(FVector2D(TimelineWidth, 0.f))
				.Size(FVector2D(TimelineWidth, TimelineHeight))
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					                .Text_Lambda([=]()
					                {
					                	const FUAFSequenceTraceInfo* UpdatedSequenceInfo = ExtractSequenceTraceInfo(StructPropertyHandle);
					                	if (UpdatedSequenceInfo == nullptr)
					                		return FText::FromString("NULL");
						                return FText::FromString(
					   FString::Printf(TEXT("%.2fs (%.0f%%)"), UpdatedSequenceInfo->CurrentTimeSeconds, UpdatedSequenceInfo->CalcAnimTimeRatio() * 100.0f));
					                })
					                .Margin(FMargin(2.0f))
					                .Justification(ETextJustify::Left)
				];

			// Todo: how to do this with lambda?
			// Add all sync markers
			if (SequenceInfo->DurationSeconds > 0.0f)
			{
				for (auto Marker : SequenceInfo->SyncMarkers)
				{
					constexpr float MarkerHeight = 8.f;
					constexpr float MarkerWidth = 4.0f;
					float MarkerTimeRatio = Marker.Time / SequenceInfo->DurationSeconds;

					TimelineCanvas->AddSlot()
					              .HAlign(HAlign_Left)
					              .VAlign(VAlign_Center)
					              .Position(FVector2D((MarkerTimeRatio * TimelineWidth) - 0.5f * MarkerWidth,
					                                  (0.5f * (TimelineHeight) + MarkerHeight)))
					              .Size(FVector2D(MarkerWidth, MarkerHeight))
					[
						SNew(SColorBlock)
						.Color(GenerateColorFromName(Marker.Name))
						.ToolTipText(FText::FromString(Marker.Name.ToString()))
					];
				}
			}

			TSharedPtr<IPropertyHandle> SequenceProperty =
				StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FUAFSequenceTraceInfo, AnimSequence));

			StructBuilder.AddCustomRow(LOCTEXT("FUAFSequenceTraceInfoRow", "FUAFSequenceTraceInfo"))
			             .NameContent()
				[
					SequenceProperty->CreatePropertyValueWidget()
				]
				.ValueContent()
				[
					TimelineCanvas
				];
		}
	}

	StructPropertyHandle->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateLambda([]
		{
			UE_LOGF(LogTemp, Display, "PropertyChange");
		}));
}

FUAFSequenceInfoTrack::FUAFSequenceInfoTrack(uint64 InObjectId)
	: ObjectId(InObjectId)
{
	Initialize();
}

FUAFSequenceInfoTrack::FUAFSequenceInfoTrack(uint64 InObjectId, uint64 InInstanceId)
	: ObjectId(InObjectId)
	, InstanceId(InInstanceId)
{
	Initialize();
}


FUAFSequenceInfoTrack::~FUAFSequenceInfoTrack()
{
	if (UUAFSequenceInfoDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get())
	{
		DetailsObject->ClearFlags(RF_Standalone);
	}
}

void FUAFSequenceInfoTrack::Initialize()
{
	ExistenceRange = MakeShared<SEventTimelineView::FTimelineEventData>();
	ExistenceRange->EventWindows.Add({0, 0, GetDisplayNameInternal(), GetDisplayNameInternal(), FLinearColor(0.1f, 0.15f, 0.11f)});

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InitializeDetailsObject();
}

UUAFSequenceInfoDetailsObject* FUAFSequenceInfoTrack::InitializeDetailsObject()
{
	UUAFSequenceInfoDetailsObject* DetailsObject = NewObject<UUAFSequenceInfoDetailsObject>();
	DetailsObject->SetFlags(RF_Standalone);
	DetailsObjectWeakPtr = MakeWeakObjectPtr(DetailsObject);
	DetailsView->SetObject(DetailsObject);
	return DetailsObject;
}

void FUAFSequenceInfoTrack::RefreshSequenceInfoFromAnimOps(TArray<FUAFSequenceTraceInfo>& OutSequenceInfo,
                                                           const FUAFInstancedAnimOpList& AnimOps)
{
	OutSequenceInfo.Reset();
	for (const FInstancedStruct& Task : AnimOps.AnimOps)
	{
		if (Task.GetScriptStruct() == FUAFDecompressAnimSequenceAnimOp::StaticStruct())
		{
			const FUAFDecompressAnimSequenceAnimOp* SequenceTask = Task.GetPtr<FUAFDecompressAnimSequenceAnimOp>();
			FUAFSequenceTraceInfo& TraceInfo = OutSequenceInfo.Emplace_GetRef();
			TraceInfo.AnimSequence = SequenceTask->GetAnimSequence();
			TraceInfo.CurrentTimeSeconds = SequenceTask->GetCurrentTime();

			if (const UAnimSequence* AnimSequence = SequenceTask->GetAnimSequence())
			{
				TraceInfo.DurationSeconds = AnimSequence->GetPlayLength();

				// Fill out sync markers
				TraceInfo.SyncMarkers.Reserve(AnimSequence->AuthoredSyncMarkers.Num());
				for (auto Marker : AnimSequence->AuthoredSyncMarkers)
				{
					TraceInfo.SyncMarkers.Emplace(Marker.Time, Marker.MarkerName);
				}
			}
		}
	}
}

bool FUAFSequenceInfoTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUAFSequenceInfoTrack::UpdateInternal);

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	constexpr bool bChanged = false;

	if (const FUAFAnimNodeProvider* UAFAnimNodeProvider = AnalysisSession->ReadProvider<FUAFAnimNodeProvider>("UAFAnimNodeProvider"))
	{
		const double CurrentScrubTime = IRewindDebugger::Instance()->CurrentTraceTime();

		UUAFSequenceInfoDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get();
		if (DetailsObject == nullptr)
		{
			// this should not happen unless the object was garbage collected (which should not happen since it's marked as Standalone)
			Initialize();
			DetailsObject = DetailsObjectWeakPtr.Get();
		}

		if (InstanceId == 0)
		{
			TraceServices::FProviderReadScopeLock UAFProviderReadScope(*UAFAnimNodeProvider);
			UAFAnimNodeProvider->EnumerateEvaluationGraphs(ObjectId, [this](uint64 GraphId)
				{
					InstanceId = GraphId;
				});
		}

		if (InstanceId != 0
			&& PreviousScrubTime != CurrentScrubTime)
		{
			PreviousScrubTime = CurrentScrubTime;

			bool bFrameFound;
			TraceServices::FFrame MarkerFrame;
			{
				TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
				const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
				bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime, MarkerFrame);
			}

			if (bFrameFound)
			{
				TraceServices::FProviderReadScopeLock UAFProviderReadScope(*UAFAnimNodeProvider);
				if (const FAnimOpData* Data = UAFAnimNodeProvider->GetAnimOpData(InstanceId))
				{
					const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
					TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);

					Data->AnimOpTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime,
						[DetailsObject, GameplayProvider, this](double InStartTime, double InEndTime, uint32 InDepth, const TArray<uint8>& VariableData)
						{
							// Why do this if panel is not selected?
							FMemoryReader Reader(VariableData);
							FObjectAsTraceIdProxyArchiveReader Archive(Reader, GameplayProvider);

							static const FUAFInstancedAnimOpList Defaults;
							FUAFInstancedAnimOpList Program;
							FUAFInstancedAnimOpList::StaticStruct()->SerializeItem(Archive, &Program, &Defaults);

							RefreshSequenceInfoFromAnimOps(DetailsObject->SequenceTraceInfo, Program);
							DetailsView->ForceRefresh(); // We need to force refresh to get the sync markers to refresh. todo: how to handle without recreating UI?
							return TraceServices::EEventEnumerate::Stop;
						});
				}
			}
		}
	}

	return bChanged;
}

bool FUAFSequenceInfoTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		if (const FUAFAnimNodeProvider* UAFAnimNodeProvider = IRewindDebugger::Instance()->GetAnalysisSession()->ReadProvider<FUAFAnimNodeProvider>("UAFAnimNodeProvider"))
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*UAFAnimNodeProvider);
			uint64 InstanceId = 0;
			UAFAnimNodeProvider->EnumerateEvaluationGraphs(InObjectId.GetMainId(), [&InstanceId](uint64 GraphId)
				{
					InstanceId = GraphId;
				});

			if (const FAnimOpData* Data = UAFAnimNodeProvider->GetAnimOpData(InstanceId))
			{
				return Data->AnimOpTimeline.GetEventCount() != 0;
			}
		}
	}

	// We don't have any AnimOps
	return false;
}
}

#undef LOCTEXT_NAMESPACE
