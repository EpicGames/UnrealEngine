// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AnimNodeCore/UAFAnimNodeDataExDetails.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "SInstancedStructPicker.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataEx.h"

#define LOCTEXT_NAMESPACE "UAFAnimNodeDataExDetails"

namespace UE::UAF::AnimNodeEditor
{
	static const TArray<TSoftObjectPtr<const UScriptStruct>>& GetBaseNodeAllowedStructs()
	{
		// Static to compute it only once
		static TArray<TSoftObjectPtr<const UScriptStruct>> CachedAllowedStructs;

		if (CachedAllowedStructs.IsEmpty())
		{
			for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
			{
				if (*StructIt != FUAFAnimNodeData::StaticStruct() && StructIt->IsChildOf<FUAFAnimNodeData>())
				{
					uint32 NumChildren = 0;

					for (const FProperty* Property = StructIt->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
					{
						if (Property->GetCPPType() == TEXT("FInstancedStruct") && Property->GetMetaData(TEXT("BaseStruct")) == TEXT("/Script/UAFAnimNode.UAFAnimNodeData"))
						{
							NumChildren++;
						}
					}

					if (NumChildren != 1)
					{
						CachedAllowedStructs.Add(*StructIt);
					}
				}
			}
		}

		return CachedAllowedStructs;
	}

	void FUAFAnimNodeDataExDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		CachedUtils = StructCustomizationUtils.GetPropertyUtilities();
		DataListHandle = InStructPropertyHandle;

		InStructPropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FUAFAnimNodeDataExDetails::OnNodeDataChanged));

		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				InStructPropertyHandle->CreatePropertyValueWidget()
			];
	}

	void FUAFAnimNodeDataExDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		check(InStructPropertyHandle->IsValidHandle());

		// Instanced structs don't have properties/children so we can't navigate them too well
		// Easier to have an editor only base node and a modifier array
		// From those, we can bake an instanced struct tree on demand
		// Base node should be constrained to those without children (e.g. sequence player)
		// Modifier array should be constrained to nodes with a single child (e.g. scale play rate)

		TSharedPtr<IPropertyHandle> BakedNodeHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FUAFAnimNodeDataEx, BakedNode));
		TSharedPtr<IPropertyHandle> BaseNodeHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FUAFAnimNodeDataEx, BaseNode));
		TSharedPtr<IPropertyHandle> ModifierNodesHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FUAFAnimNodeDataEx, ModifierNodes));

		// The baked result
		StructBuilder.AddProperty(BakedNodeHandle.ToSharedRef());

		// The base node
		{
			IDetailPropertyRow& Row = StructBuilder.AddProperty(BaseNodeHandle.ToSharedRef());

			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			Row.GetDefaultWidgets(NameWidget, ValueWidget);

			Row.CustomWidget(true)
				.NameContent()
				.HAlign(HAlign_Fill)
				[
					NameWidget.ToSharedRef()
				]
				.ValueContent()
				.HAlign(HAlign_Fill)
				[
					SNew(SInstancedStructPicker, BaseNodeHandle, StructCustomizationUtils.GetPropertyUtilities())
						.OnStructPicked_Lambda([this](const UScriptStruct* InStruct)
							{
								OnNodeDataChanged();
							})
						.AllowedStructs(GetBaseNodeAllowedStructs())
				];
		}

		// The modifier node array
		{
			TSharedRef<FDetailArrayBuilder> ArrayBuilder = MakeShared<FDetailArrayBuilder>(ModifierNodesHandle.ToSharedRef());
			ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FUAFAnimNodeDataExDetails::GenerateNodeDataWidget));
			StructBuilder.AddCustomBuilder(ArrayBuilder);
		}
	}

	void FUAFAnimNodeDataExDetails::GenerateNodeDataWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
	{
		check(CachedUtils);

		IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(PropertyHandle);

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		Row.GetDefaultWidgets(NameWidget, ValueWidget);

		Row.CustomWidget(true)
			.NameContent()
			.HAlign(HAlign_Fill)
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SInstancedStructPicker, PropertyHandle, CachedUtils)
					.OnStructPicked_Lambda([this](const UScriptStruct* InStruct)
						{
							OnNodeDataChanged();
						})
			];
	}

	void FUAFAnimNodeDataExDetails::OnNodeDataChanged()
	{
		check(DataListHandle);

		FScopedTransaction ScopedTransaction(LOCTEXT("UAFRefreshNodeData", "Refresh Node Data"));
		DataListHandle->NotifyPreChange();
		DataListHandle->EnumerateRawData([](void* RawData, const int32 DataIndex, const int32 NumDatas)
			{
				FUAFAnimNodeDataEx* DataList = static_cast<FUAFAnimNodeDataEx*>(RawData);
				DataList->Refresh();

				return true;
			});
		DataListHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		DataListHandle->NotifyFinishedChangingProperties();
		
	}
}

#undef LOCTEXT_NAMESPACE
