// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

namespace UE::UAF::AnimNodeEditor
{
	class FUAFAnimNodeDataExDetails : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance()
		{
			return MakeShareable(new FUAFAnimNodeDataExDetails());
		}

		// IPropertyTypeCustomization interface
		virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	private:
		void GenerateNodeDataWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder);
		void OnNodeDataChanged();

		TSharedPtr<IPropertyHandle> DataListHandle;
		TSharedPtr<IPropertyUtilities> CachedUtils;
	};
}
