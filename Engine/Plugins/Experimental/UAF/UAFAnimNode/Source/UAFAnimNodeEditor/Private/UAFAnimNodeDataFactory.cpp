// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimNodeDataFactory.h"

#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "Templates/SubclassOf.h"

namespace UE::UAF
{
TMap<FTopLevelAssetPath, FUAFAnimNodeDataFactory::FInitializerInfo> FUAFAnimNodeDataFactory::AssetInitializers;
TArray<FTopLevelAssetPath> FUAFAnimNodeDataFactory::AllSupportedObjectClassPaths;

const FUAFAnimNodeDataFactory::FInitializerInfo* FUAFAnimNodeDataFactory::FindAssetInitializer(const UClass* Class)
{
	check(Class != nullptr);
	FInitializerInfo* InitializerInfo;
	const UClass* SearchClass = Class;
	do
	{
		InitializerInfo = AssetInitializers.Find(SearchClass->GetClassPathName());
		SearchClass = SearchClass->GetSuperClass();
	}
	while (InitializerInfo == nullptr && SearchClass != nullptr);
	return InitializerInfo;
}

FTopLevelAssetPath FUAFAnimNodeDataFactory::RegisterAsset(TSubScriptStructOf<FUAFAnimNodeData> AnimNodeDataType, UClass* ObjectClass, FAssetInitializer&& Initializer)
{
	const FTopLevelAssetPath ClassPathName = ObjectClass->GetClassPathName();
	
	// todo: we may want to support this for licensee overrides
	checkf(AssetInitializers.Contains(ClassPathName) == false, TEXT("Attempting to register type %s in UAF::FUAFAnimNodeDataFactory twice"), *ObjectClass->GetFullName());

	AssetInitializers.Add(ClassPathName, FInitializerInfo(AnimNodeDataType, Initializer));
	AllSupportedObjectClassPaths.Add(ClassPathName);
	return ClassPathName;
}

TInstancedStruct<FUAFAnimNodeData> FUAFAnimNodeDataFactory::CreateUAFAnimNodeDataFromObject(UObject* Object)
{
	check(Object != nullptr);
	
	const FInitializerInfo* InitializerInfo = FindAssetInitializer(Object->GetClass());
	if (ensureMsgf(InitializerInfo != nullptr, TEXT("Unsupported object type (%s) passed to UAF::FAnimNodeFactory"), *Object->GetClass()->GetFullName()))
	{
		return InitializerInfo->Initializer(Object);
	}

	return TInstancedStruct<FUAFAnimNodeData>();
}

void FUAFAnimNodeDataFactory::UnregisterAsset(const FTopLevelAssetPath& InObjectClassPath)
{
	check(InObjectClassPath.IsValid());

	int Removed = AssetInitializers.Remove(InObjectClassPath);
	ensure(Removed > 0);
	Removed = AllSupportedObjectClassPaths.Remove(InObjectClassPath);
	ensure(Removed > 0);
}

UAFANIMNODEEDITOR_API bool FUAFAnimNodeDataFactory::IsAssetRegistered(const UClass* AssetClass)
{
	return FindAssetInitializer(AssetClass) != nullptr;
}

TArray<UClass*> FUAFAnimNodeDataFactory::GetRegisteredObjectClasses()
{
	TArray<UClass*> AllSupportedObjectClasses;
	AllSupportedObjectClasses.Reserve(AllSupportedObjectClassPaths.Num());
	for (const FTopLevelAssetPath& ClassName : AllSupportedObjectClassPaths)
	{
		if (UClass* Class = FindObject<UClass>(nullptr, ClassName.ToString()))
		{
			AllSupportedObjectClasses.Add(Class);
		}
	}
	return AllSupportedObjectClasses;
}

TArray<UClass*> FUAFAnimNodeDataFactory::GetRegisteredObjectClasses(TSubScriptStructOf<FUAFAnimNodeData> BaseAssetType)
{
	TArray<UClass*> CompatibleClasses;
	CompatibleClasses.Reserve(AllSupportedObjectClassPaths.Num());

	for (auto Info : AssetInitializers)
	{
		if (Info.Value.AnimNodeDataType->IsChildOf(BaseAssetType.Get()))
		{
			if (UClass* Class = FindObject<UClass>(nullptr, Info.Key.ToString()))
			{
				CompatibleClasses.Add(Class);
			}
		}
	}

	return CompatibleClasses;
}
}


