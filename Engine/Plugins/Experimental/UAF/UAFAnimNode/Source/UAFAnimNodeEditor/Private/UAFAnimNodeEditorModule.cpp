// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAnimNodeEditorModule.h"

#include "Features/IModularFeatures.h"
#include "IAnimNextEditorModule.h"
#include "IWorkspaceEditorModule.h"
#include "IWorkspaceEditor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RewindDebugger/UAFAnimNodeTraceModule.h"
#include "RewindDebugger/UAFAnimOpTrack.h"
#include "RewindDebugger/UAFSequenceInfoTrack.h"
#include "UAFAnimNodeDataFactory.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataEx.h"
#include "UAF/AnimNodeCore/UAFAnimNodeDataExDetails.h"
#include "UAF/AnimNodes/UAFSequencePlayer.h"
#include "RewindDebugger/UAFAnimNodeTrack.h"

#define LOCTEXT_NAMESPACE "UAFAnimNodeEditorModule"

namespace
{
	FUAFAnimNodeTraceModule GUAFAnimNodeTraceModule;
	UE::UAF::Editor::FUAFAnimOpTrackCreator GUAFModulesTrackCreator;
	UE::UAF::Editor::FUAFSequenceInfoTrackCreator GUAFSequenceInfoTrackCreator;
	UE::UAF::Editor::FUAFAnimNodeTrackCreator GUAFAnimNodeTrackCreator;
	UE::UAF::Editor::FUAFTransitionTrackCreator GUAFTransitionTrackCreator;
}

namespace UE::UAF::AnimNodeEditor
{

void FModule::StartupModule()
{
	AnimSequenceClassPath = FUAFAnimNodeDataFactory::RegisterAsset<FUAFSequencePlayerData, UAnimSequence>(
		[](UAnimSequence* AnimSequence)
			{
				FUAFSequencePlayerData SequencePlayer;
				SequencePlayer.Sequence = AnimSequence;
				return SequencePlayer;
			});

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertiesToUnregisterOnShutdown.Add(FUAFSequenceTraceInfo::StaticStruct()->GetFName());
	PropertyModule.RegisterCustomPropertyTypeLayout(
		PropertiesToUnregisterOnShutdown.Last(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<UE::UAF::Editor::FUAFSequenceTraceInfoCustomization>(); }));

	PropertiesToUnregisterOnShutdown.Add(FUAFAnimNodeDataEx::StaticStruct()->GetFName());
	PropertyModule.RegisterCustomPropertyTypeLayout(PropertiesToUnregisterOnShutdown.Last(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FUAFAnimNodeDataExDetails::MakeInstance));

	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &GUAFAnimNodeTraceModule);
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFModulesTrackCreator);
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFSequenceInfoTrackCreator);
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFAnimNodeTrackCreator);
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFTransitionTrackCreator);
}

void FModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFSequenceInfoTrackCreator);
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFModulesTrackCreator);
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFAnimNodeTrackCreator);
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GUAFTransitionTrackCreator);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &GUAFAnimNodeTraceModule);

	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (const FName& PropertyName : PropertiesToUnregisterOnShutdown)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
	}

	FUAFAnimNodeDataFactory::UnregisterAsset(AnimSequenceClassPath);
}
	
}

IMPLEMENT_MODULE(UE::UAF::AnimNodeEditor::FModule, UAFAnimNodeEditor);

#undef LOCTEXT_NAMESPACE