// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/SubScriptStructOf.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"

namespace UE::UAF
{
	
struct FUAFAnimNodeData;

/*
 * The FUAFAnimNodeDataFactory is used to produce FUAFAnimNodeData from UObjects. This is used for UI interactions 
 * in editor when users drop objects on to FUAFAnimNodeData slots
 *
 * This factory links UObject class types to FUAFAnimNodeData types.
 *
 * Note: It is not a requirement for a FUAFAnimNodeData be created from a UObject. It could also be created manually
 */
struct FUAFAnimNodeDataFactory
{
public:

	// Create a UAFAnimNodeData object from the supplied Object
	static UAFANIMNODEEDITOR_API TInstancedStruct<FUAFAnimNodeData> CreateUAFAnimNodeDataFromObject(UObject* Object);

	template<typename TAnimNodeDataType, typename TObjectType>
	using TAnimNodeDataInitializer = TFunction<TAnimNodeDataType(TObjectType*)>;

	// Register an UAF asset with the factory
	// This defines a relationship between the TObjectType and the TAnimNodeDataType.
	// The initializer definnes how to create a TAnimNodeDataType from a TObjectType instance
	// @return a FTopLevelAssetPath that can be used to unregister the asset type via UnregisterAsset
	template<typename TAnimNodeDataType, typename TObjectType>
	[[nodiscard]] static FTopLevelAssetPath RegisterAsset(TAnimNodeDataInitializer<TAnimNodeDataType, TObjectType>&& Initializer)
	{
		static_assert(std::is_base_of_v<FUAFAnimNodeData, TAnimNodeDataType>); // Check for invalid types. TAnimNodeDataType must derive from FUAFAnimNodeData
		return RegisterAsset(TAnimNodeDataType::StaticStruct(), TObjectType::StaticClass(), [Initializer](UObject* ObjectIn)->TInstancedStruct<FUAFAnimNodeData>
			{
				return TInstancedStruct<TAnimNodeDataType>::Make(Initializer(Cast<TObjectType>(ObjectIn)));
			});
	}

	// Unregister an UAF asset with the factory
	static UAFANIMNODEEDITOR_API void UnregisterAsset(const FTopLevelAssetPath& InObjectClassPath);

	// Checks if an asset type is registered with the factory
	static UAFANIMNODEEDITOR_API bool IsAssetRegistered(const UClass* AssetClass);

	// Get all registered UClass types
	static UAFANIMNODEEDITOR_API TArray<UClass*> GetRegisteredObjectClasses();

	// Get registered UClass types that correspond to a specific baseclass of FUAFAnimNodeData
	static UAFANIMNODEEDITOR_API TArray<UClass*> GetRegisteredObjectClasses(TSubScriptStructOf<FUAFAnimNodeData> BaseAssetType);

	template<typename TAnimNodeDataType>
	static TArray<UClass*> GetRegisteredObjectClasses()
	{
		static_assert(std::is_base_of_v<FUAFAnimNodeData, TAnimNodeDataType>);
		return GetRegisteredObjectClasses(TAnimNodeDataType::StaticStruct());
	}
private:

	// Untyped UObject initializer
	using FAssetInitializer = TFunction<TInstancedStruct<FUAFAnimNodeData>(UObject*)>;

	struct FInitializerInfo
	{
		FInitializerInfo(TSubScriptStructOf<FUAFAnimNodeData> Type, FAssetInitializer Initializer)
			: AnimNodeDataType(Type), Initializer(Initializer){}

		FInitializerInfo() = default;
		
		TSubScriptStructOf<FUAFAnimNodeData> AnimNodeDataType;
		FAssetInitializer Initializer;
	};
	
	
	// Register how to create a FUAFAnimNodeData from a ObjectClass
	static UAFANIMNODEEDITOR_API FTopLevelAssetPath RegisterAsset(TSubScriptStructOf<FUAFAnimNodeData> AnimNodeDataType, UClass* ObjectClass, FAssetInitializer&& Initializer);
	static UAFANIMNODEEDITOR_API const FInitializerInfo* FindAssetInitializer(const UClass* Class);
	static TMap<FTopLevelAssetPath, FInitializerInfo> AssetInitializers;
	static TArray<FTopLevelAssetPath> AllSupportedObjectClassPaths;
};
	
}
