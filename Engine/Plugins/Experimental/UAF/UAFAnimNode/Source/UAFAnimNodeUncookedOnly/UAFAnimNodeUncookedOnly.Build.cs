// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFAnimNodeUncookedOnly : ModuleRules
	{
		public UAFAnimNodeUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"UAFAnimNode",
				}
			);
		}
	}
}
