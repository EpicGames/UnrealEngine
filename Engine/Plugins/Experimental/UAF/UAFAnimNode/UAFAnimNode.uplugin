{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "UAF Anim Node",
	"Description" : "Nodes system for UAF.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins": [
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "UAFAnimNode",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFAnimNodeEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFAnimNodeUncookedOnly",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		}
	]
}
